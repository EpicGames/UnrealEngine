// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CookerStats.generated.h"


/** Enum defining the object sets for this stats object */
UENUM()
enum ECookerStatsObjectSets : int
{
	CookerStatsObjectSets_Default UMETA(DisplayName="Default", ToolTip="View cooker statistics"),
};


/**
 * Statistics for a cooked asset.
 *
 * Note: We assume that asset files are not larger than 2GB, because the StatsViewer is still lacking int64 support.
 */
UCLASS(Transient, MinimalAPI, meta=(DisplayName="Cooker Stats", ObjectSetType="ECookerStatsObjectSets"))
class UCookerStats
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** The assets contained in the file. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=(DisplayName="Asset(s)", ColumnWidth="100"))
	TArray<TWeakObjectPtr<UObject>> Assets;

	/** The size of the assets before cooking. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=(DisplayName="Size (Original)", ColumnWidth="50", ShowTotal="true", Unit="KB"))
	float SizeBefore;

	/** The size of the assets after cooking. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=(DisplayName="Size (Cooked)", ColumnWidth="50", ShowTotal="true", Unit="KB"))
	float SizeAfter;

	/** Asset path without the name "package.[group.]" */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=(ColumnWidth="300"))
	FString Path;
};
