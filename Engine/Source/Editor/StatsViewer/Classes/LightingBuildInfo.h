// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "LightingBuildInfo.generated.h"

/** Enum defining the object sets for this stats object */
UENUM()
enum ELightingBuildInfoObjectSets : int
{
	LightingBuildInfoObjectSets_Default				UMETA( DisplayName = "Default" , ToolTip = "View lighting build statistics" ),
};

/** Statistics page for lighting. */
UCLASS(Transient, MinimalAPI, meta=( DisplayName = "Lighting Build Info", ObjectSetType = "ELightingBuildInfoObjectSets" ) )
class ULightingBuildInfo : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The actor and/or object that is related to this info. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "150" ) )
	TWeakObjectPtr<UObject> Object;

	/** The lighting time this object took. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "200", ShowTotal = "true", Unit = "s" ) )
	float LightingTime;

	/** The percentage of unmapped texels for this object. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Unmapped Texels", ShowTotal = "true", ColumnWidth = "142", Unit = "%" ) )
	float UnmappedTexelsPercentage;

	/** The memory consumed by unmapped texels for this object, in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "194", ShowTotal = "true", Unit = "KB" ) )
	float UnmappedTexelsMemory;

	/** The memory consumed by all texels for this object, in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "220", ShowTotal = "true", SortMode = "Descending", Unit = "KB" ) )
	float TotalTexelMemory;

	/** The name of the level this object resides in */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "168" ) )
	FString LevelName;

public:

	/** 
	 * Set the values for this stats entry 
	 * @param	Object						The object this lighting info relates to
	 * @param	LightingTime				The time in seconds that this objects lighting build took
	 * @param	UnmappedTexelsPercentage	The percentage of unmapped texels for this object
	 * @param	UnmappedTexelsMemory		The memory consumed by unmapped texels for this object, in bytes
	 * @param	TotalTexelMemory			The memory consumed by all texels for this object, in bytes
	 */
	virtual void Set( TWeakObjectPtr<UObject> Object, double LightingTime, float UnmappedTexelsPercentage, int32 UnmappedTexelsMemory, int32 TotalTexelMemory );

private:

	/** Update internal strings */
	void UpdateNames();
};
