// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "PrimitiveStats.generated.h"

class AActor;

/** Enum defining the object sets for this stats object */
UENUM()
enum EPrimitiveObjectSets : int
{
	PrimitiveObjectSets_AllObjects			UMETA( DisplayName = "All Objects" , ToolTip = "View primitive statistics for all objects in all levels" ),
	PrimitiveObjectSets_CurrentLevel		UMETA( DisplayName = "Current Level" , ToolTip = "View primitive statistics for objects in the current level" ),
	PrimitiveObjectSets_SelectedObjects		UMETA( DisplayName = "Selected Objects" , ToolTip = "View primitive statistics for selected objects" ),
};

/** Statistics page for primitives.  */
UCLASS(Transient, MinimalAPI, meta=( DisplayName = "Primitive Stats", ObjectSetType = "EPrimitiveObjectSets" ) )
class UPrimitiveStats : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Resource (e.g. UStaticMesh, USkeletalMesh, UModelComponent, UTerrainComponent, etc */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Object", ColumnWidth = "200" ) )
	TWeakObjectPtr<UObject>	Object;

	/** Actor(s) that use the resource - click to select & zoom Actor(s) */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Actor(s)", ColumnWidth = "200" ) )
	TArray< TWeakObjectPtr<AActor> > Actors;

	/** Type name*/
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Type", ColumnWidth = "200" ) )
	FString Type;

	/** Number of occurrences in map */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "86", ShowTotal = "true" ) )
	int32 Count;

	/** Section count of mesh */
	UPROPERTY()
	int32 Sections;

	/** Hardware instances */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "102", ShowTotal = "true"))
	int32 HWInstances;

	/** Instanced section count of mesh */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "102", ShowTotal = "true" ) )
	int32 InstSections;

	/** Triangle count of mesh */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Tris", ColumnWidth = "74", ShowTotal = "true" ) )
	int32 Triangles;

	/** Triangle count of all mesh occurances (Count * Tris) */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Sum Tris", ColumnWidth = "104", ShowTotal = "true" ) )
	int32 InstTriangles;

	/** Resource size in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Size", ColumnWidth = "78", ShowTotal = "true", SortMode = "Descending", Unit = "KB" ) )
	float ResourceSize;

	/** Vertex color stat for static and skeletal meshes in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "VC", ColumnWidth = "68", ShowTotal = "true", Unit = "KB" ) )
	float VertexColorMem;

	/** Per component vertex color stat for static meshes in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Inst VC", ColumnWidth = "94", ShowTotal = "true", Unit = "KB" ) )
	float InstVertexColorMem;

	/** Average number of lightmap lights relevant to each instance */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Avg LM", ColumnWidth = "96", ShowTotal = "true" ) )
	int32 LightsLM;

	/** Average number of other lights relevant to each instance */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Avg OL", ColumnWidth = "94", ShowTotal = "true" ) )
	float LightsOther;

	/** (Avg OL + Avg LM) / Count */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Sum Avg", ColumnWidth = "104", ShowTotal = "true" ) )
	float LightsTotal;

	/** Avg OL * Sections */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Cost", ColumnWidth = "78", ShowTotal = "true" ) )
	float ObjLightCost;

	/** Light map data in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "LM", ColumnWidth = "70", ShowTotal = "true", Unit = "KB" ) )
	float LightMapData;

	/** Light/shadow map resolution */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Res", ColumnWidth = "74", ShowTotal = "true" ) )
	float LMSMResolution;

	/** Minimum radius of bounding sphere of instance in map */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Min R", ColumnWidth = "84", ShowTotal = "true" ) )
	double RadiusMin;

	/** Maximum radius of bounding sphere of instance in map */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Max R", ColumnWidth = "88", ShowTotal = "true" ) )
	double RadiusMax;

	/** Average radius of bounding sphere of instance in map */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Avg R", ColumnWidth = "86", ShowTotal = "true" ) )
	double RadiusAvg;

public:

	/** Update internal strings */
	void UpdateNames();
};
