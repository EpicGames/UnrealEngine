// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "ShaderCookerStats.generated.h"


UENUM()
enum EShaderCookerStatsSets : int
{
	EShaderCookerStatsSets_Default				UMETA(DisplayName = "Default", ToolTip = "Shader Cooker Sets"),
};

/** Statistics page for shader cooker stats. */
UCLASS(Transient, MinimalAPI, meta=( DisplayName = "Shader Cooker Stats", ObjectSetType = "EShaderCookerStatsSets" ) )
class UShaderCookerStats : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Material name */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "200"))
	FString Name;

	/** Material platform */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "50"))
	FString Platform;

	/** Material Category*/
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "50", ToolTip = "Material Category. Edit Engine/Config/ShaderConfiguration.csv & Engine/Config/ShaderConfiguration.csv to configure"))
	FString Category;

	/** Number of shaders compilations */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Compiled", ToolTip = "Number of shaders that was actually compiled and not present in the DDC", ColumnWidth = "74", ShowTotal = "true"))
	int32 Compiled;

	/** Number of shaders cooked*/
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Cooked", ToolTip = "Total number of shaders cooked", ColumnWidth = "74", ShowTotal = "true"))
	int32 Cooked;

	/** Number of permutations */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Permutations", ToolTip = "Number of material permutations generated by switch etc.",  ColumnWidth = "74", ShowTotal = "true"))
	int32 Permutations;

	/** Time spent compiling */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Compile Time", ToolTip = "Cpu time spent compiling shaders not in DDC", ColumnWidth = "74", ShowTotal = "true"))
	float CompileTime ;

	/** Material path */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "200"))
	FString Path;


	int32 CategoryIndex;
};
