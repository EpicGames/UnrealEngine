// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "StaticMeshLightingInfo.generated.h"

class AActor;
class UStaticMesh;
class UStaticMeshComponent;

/** Enum defining the object sets for this stats object */
UENUM()
enum EStaticMeshLightingInfoObjectSets : int
{
	StaticMeshLightingInfoObjectSets_CurrentLevel		UMETA( DisplayName = "Current Level" , ToolTip = "View static mesh lighting info for the current level" ),
	StaticMeshLightingInfoObjectSets_SelectedLevels		UMETA( DisplayName = "Selected Levels" , ToolTip = "View lighting info for selected levels" ),
	StaticMeshLightingInfoObjectSets_AllLevels			UMETA( DisplayName = "All Levels" , ToolTip = "View static mesh lighting info for all levels" ),
};

/** Statistics page for static meshes. */
UCLASS(Transient, MinimalAPI, meta=( DisplayName = "Static Mesh Lighting Info", ObjectSetType = "EStaticMeshLightingInfoObjectSets" ) )
class UStaticMeshLightingInfo : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The actor that is related to this error/warning. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Actor", ColumnWidth = "200" ) )
	TWeakObjectPtr<AActor> StaticMeshActor;

	/** The source StaticMesh that is related to this info. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "200" ) )
	TWeakObjectPtr<UStaticMesh> StaticMesh;

	/** Cached version of the level name this object resides in */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Level", ColumnWidth = "150" ) )
	FString LevelName;

	/** The staticmesh component that is related to this info. */
	TWeakObjectPtr<UStaticMeshComponent> StaticMeshComponent;

	/** Current mapping type string */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Type", ColumnWidth = "82" ) )
	FString TextureMapping;

	/** Current mapping type flag - not displayed */
	UPROPERTY()
	bool bTextureMapping;

	/** Does the Lightmap have UVs? */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "UVs", ColumnWidth = "76" ) )
	bool bHasLightmapTexCoords;

	/** The static lighting resolution the texture mapping was estimated with. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Res", ColumnWidth = "74" ) )
	int32 StaticLightingResolution;

	/** Estimated memory usage in KB for light map texel data. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Texture LM", ShowTotal = "true", ColumnWidth = "118", Unit = "KB" ) )
	float TextureLightMapMemoryUsage;

	/** Estimated memory usage in KB for light map vertex data. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Vertex LM", ShowTotal = "true", ColumnWidth = "112", Unit = "KB" ) )
	float VertexLightMapMemoryUsage;

	/** Num lightmap lights */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Num LM", ShowTotal = "true", ColumnWidth = "100" ) )
	int32 LightMapLightCount;

	/** Estimated memory usage in KB for shadow map texel data. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Texture SM", ShowTotal = "true", ColumnWidth = "120", SortMode = "Descending", Unit = "KB" ) )
	float TextureShadowMapMemoryUsage;

	/** Estimated memory usage in KB for shadow map vertex data. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Vertex SM", ShowTotal = "true", ColumnWidth = "112", Unit = "KB" ) )
	float VertexShadowMapMemoryUsage;

	/** Number of lights generating shadow maps on the primitive. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Num SM", ShowTotal = "true", ColumnWidth = "102" ) )
	int32 ShadowMapLightCount;

	/** The lighting texture names that the static mesh component references. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Lightmap Names", ColumnWidth = "300"))
	TArray<FString> LightmapTextureNames;

public:

	/** Update internal strings */
	void UpdateNames();
};
