// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "PixelFormat.h"
#include "Engine/TextureDefines.h"
#include "TextureStats.generated.h"

class AActor;
class UTexture;

/** Enum defining the object sets for this stats object */
UENUM()
enum ETextureObjectSets : int
{
	TextureObjectSet_CurrentStreamingLevel	UMETA( DisplayName = "Current Streaming Level" , ToolTip = "Display texture statistics for the current streaming level" ),
	TextureObjectSet_AllStreamingLevels		UMETA( DisplayName = "All Streaming Levels" , ToolTip = "Display texture statistics for all streaming levels" ),
	TextureObjectSet_SelectedActors			UMETA( DisplayName = "Selected Actor(s)" , ToolTip = "Display texture statistics of selected Actors" ),
	TextureObjectSet_SelectedMaterials		UMETA( DisplayName = "Selected Materials(s)" , ToolTip = "Display texture statistics of selected Materials" ),

	// @todo: These two stat sets are deprecated as UE doesnt support them currently
	// To recreate them, you will need to re-implement the functionality left behind in
	// Engine\Source\Editor\Stats\Private\TextureInfo.cpp
//	TextureObjectSet_CookerStatistics		UMETA( DisplayName = "Cooker Statistics" , ToolTip = "Display texture statistics from a cooker statistics file (.upk)" ),
//	TextureObjectSet_RemoteCapture			UMETA( DisplayName = "Remote Capture" , ToolTip = "Display texture statistics from a remote capture" ),
};

/** Statistics page for textures. */
UCLASS(Transient, MinimalAPI, meta=( DisplayName = "Texture Stats", ObjectSetType = "ETextureObjectSets" ) )
class UTextureStats : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Texture - click to go to asset */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Name", ColumnWidth = "100" ) )
	TWeakObjectPtr<UTexture> Texture;

	/** Actor(s) - click to select & zoom Actor(s) */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Actor(s)", ColumnWidth = "100" ) )
	TArray< TWeakObjectPtr<AActor> > Actors;

	/** Texture type e.g. 2D, 3D, Cube, "" if not known, ... */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "60" ) )
	FString Type;

	/** Is the texture virtual? Yes/No */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "60"))
	FString Virtual;

	/** Max Dimension e.g. 256x256, not including the format */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Max Dimension", ColumnWidth = "90" ) )
	FVector2D MaxDim;

	/** Current Dimension e.g 256x256 */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Current Dimension", ColumnWidth = "90" ) )
	FVector2D CurrentDim;

	/** The texture format, e.g. PF_DXT1 */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "96" ) )
	TEnumAsByte<EPixelFormat> Format;

	/** The texture group, TEXTUREGROUP_MAX is not used, e.g. TEXTUREGROUP_World */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "94" ) )
	TEnumAsByte<TextureGroup> Group;

	/** Combined LOD Bias for this texture. (Texture LODBias + Texture group) */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "CombinedLODBias", ColumnWidth = "50" ) )
	int32 CombinedLODBias;

	/** LOD Bias for this texture. */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "LODBias", ColumnWidth = "50" ) )
	int32 TextureLODBias;

	/** The memory used currently in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Current Memory", ColumnWidth = "80", ShowTotal = "true", Unit = "KB" ) )
	float CurrentKB;

	/** The memory used when the texture is fully loaded in KB*/
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=(DisplayName = "Fully Loaded Memory",  ColumnWidth = "110", ShowTotal = "true", SortMode = "Descending", Unit = "KB" ) )
	float FullyLoadedKB;

	/** The number of times the texture is used */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Uses", ColumnWidth = "74", ShowTotal = "true" ) )
	int32 NumUses;

	/** Relative time it was used for rendering the last time */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( DisplayName = "Last Seen", ColumnWidth = "78", Unit = "s" ) )
	float LastTimeRendered;

	/** Texture path without the name "package.[group.]" */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category="Stats", meta=( ColumnWidth = "200" ) )
	FString Path;
};
