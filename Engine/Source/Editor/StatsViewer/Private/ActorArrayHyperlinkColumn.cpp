// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorArrayHyperlinkColumn.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "StatsCellPresenter.h"
#include "Editor.h"
#include "PropertyHandle.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "PropertyPath.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer"

class FActorArrayHyperlinkCellPresenter : public TSharedFromThis< FActorArrayHyperlinkCellPresenter >, public FStatsCellPresenter
{
public:
	FActorArrayHyperlinkCellPresenter( TSharedPtr<IPropertyHandleArray> InActors ) :
		Actors( InActors )
	{
		uint32 NumElements = 0;
		Actors->GetNumElements(NumElements);

		if(NumElements == 1)
		{
			TSharedRef<IPropertyHandle> FirstElement = Actors->GetElement(0);
			UObject* Object = NULL;
			FirstElement->GetValue(Object);
			if(Object != NULL)
			{
				Text = FText::FromString( Object->GetName() );
			}
			else
			{
				Text = LOCTEXT("ActorArrayHyperlinkNone", "None");
			}
		}
		else if(NumElements > 1)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("NumActors"), NumElements);

			Text = FText::Format(LOCTEXT("ActorArrayHyperlinkMany", "{NumActors} Actors"), Arguments);
		}
		else
		{
			Text = LOCTEXT("ActorArrayHyperlinkNone", "None");
		}
	}

	virtual ~FActorArrayHyperlinkCellPresenter() {}

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override
	{
		return SNew( SHorizontalBox )
			
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SHyperlink )
					.Text( GetValueAsText() )
					.OnNavigate( this, &FActorArrayHyperlinkCellPresenter::OnHyperlinkClicked )
					.Style(FAppStyle::Get(), "DarkHyperlink")
			];
	}

	void OnHyperlinkClicked( )
	{
		uint32 NumElements = 0;
		Actors->GetNumElements(NumElements);
		if (NumElements > 0)
		{
			TArray<AActor*> ActorsToFocus;
			TArray<UObject*> ObjectsToSync;

			const FScopedTransaction Transaction( LOCTEXT("SelectActors", "Statistics Select Actors") );
			GEditor->GetSelectedActors()->BeginBatchSelectOperation();
			GEditor->SelectNone(/*bNotify*/false, /*bDeselectBSPSurfs*/true);
			for(uint32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
			{
				TSharedRef<IPropertyHandle> Element = Actors->GetElement(ElementIndex);
				UObject* Object = NULL;
				Element->GetValue(Object);
				if(Object != NULL)
				{
					AActor* Actor = Cast<AActor>(Object);

					if (Actor != nullptr)
					{
						GEditor->SelectActor(Actor, /*bInSelected*/true, /*bNotify*/false, /*bSelectEvenIfHidden*/true);
						ActorsToFocus.Add(Actor);
					}
					else
					{
						ObjectsToSync.Add(Object);
					}
				}
			}

			GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify*/true);

			if(ActorsToFocus.Num() > 0)
			{
				GEditor->MoveViewportCamerasToActor(ActorsToFocus, false);	
			}

			if (ObjectsToSync.Num() > 0)
			{
				GEditor->SyncBrowserToObjects(ObjectsToSync);
			}
		}
	}

private:

	/** The actor(s) we will link to */
	TSharedPtr<IPropertyHandleArray> Actors;
};

bool FActorArrayHyperlinkColumn::Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const
{
	if( Column->GetDataSource()->IsValid() )
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			const FProperty* Property = PropertyInfo.Property.Get();
			if( Property->IsA( FArrayProperty::StaticClass() ) )
			{
				const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
				if( ArrayProperty->Inner->IsA(FWeakObjectProperty::StaticClass()) )
				{
					return true;
				}
			}
		}
	}

	return false;
}

TSharedPtr< SWidget > FActorArrayHyperlinkColumn::CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	return NULL;
}

TSharedPtr< IPropertyTableCellPresenter > FActorArrayHyperlinkColumn::CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if( PropertyHandle.IsValid() )
	{
		TSharedPtr<IPropertyHandleArray> PropertyArray = PropertyHandle->AsArray();
		if( PropertyArray.IsValid() )
		{
			return MakeShareable( new FActorArrayHyperlinkCellPresenter(PropertyArray) );
		}
	}

	return NULL;
}

#undef LOCTEXT_NAMESPACE
