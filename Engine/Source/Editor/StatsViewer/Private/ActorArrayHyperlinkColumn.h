// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableUtilities.h"
#include "IPropertyTableCustomColumn.h"

class IPropertyTableCell;
class IPropertyTableCellPresenter;

/**
 * A property table custom column used to display names of objects
 * that can be clicked on to jump to the objects in the scene or 
 * content browser.
 */
class FActorArrayHyperlinkColumn : public IPropertyTableCustomColumn
{
public:
	/** Begin IPropertyTableCustomColumn interface */
	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const override;
	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	/** End IPropertyTableCustomColumn interface */
};

