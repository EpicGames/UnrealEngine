// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectHyperlinkColumn.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "Engine/Texture.h"
#include "Engine/StaticMesh.h"
#include "StatsCellPresenter.h"
#include "Editor.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "PropertyPath.h"
#include "ScopedTransaction.h"
#include "StatsViewerUtils.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer"

class FObjectHyperlinkCellPresenter : public TSharedFromThis< FObjectHyperlinkCellPresenter >, public FStatsCellPresenter
{
public:

	FObjectHyperlinkCellPresenter( TWeakObjectPtr<UObject> InObject, const FOnGenerateWidget& InOnGenerateWidget ) 
		: Object( InObject )
		, OnGenerateWidget(InOnGenerateWidget)
	{
		if(InObject.IsValid())
		{
			Text = StatsViewerUtils::GetAssetName( InObject );
		}
		else
		{
			Text = LOCTEXT("PrimitiveHyperlinkNone", "<None>");
		}
	}

	virtual ~FObjectHyperlinkCellPresenter() {}

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override
	{
		return OnGenerateWidget.Execute(GetValueAsText(), Object);
	}

private:

	/** The object we will link to */
	TWeakObjectPtr< UObject > Object;

	/** Delegate used to generate a custom widget */
	FOnGenerateWidget OnGenerateWidget;
};

FObjectHyperlinkColumn::FObjectHyperlinkColumn(const FObjectHyperlinkColumnInitializationOptions& InOptions)
	: OnGenerateWidget(InOptions.OnGenerateWidget)
	, OnObjectHyperlinkClicked(InOptions.OnObjectHyperlinkClicked)
	, OnIsClassSupported(InOptions.OnIsClassSupported)
{
	struct Local
	{
		static void HandleNavigate(TWeakObjectPtr<UObject> InObjectPtr, FOnObjectHyperlinkClicked OnObjectHyperlinkClicked)
		{
			OnObjectHyperlinkClicked.ExecuteIfBound(InObjectPtr);
		}

		static TSharedRef<SWidget> HandleGenerateWidget(const FText& InValueAsText, const TWeakObjectPtr<UObject>& InObjectPtr, FOnObjectHyperlinkClicked OnObjectHyperlinkClicked)
		{
			return 
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SHyperlink )
					.Text( InValueAsText )
					.OnNavigate( FSimpleDelegate::CreateStatic(&Local::HandleNavigate, InObjectPtr, OnObjectHyperlinkClicked) )
					.Style(FAppStyle::Get(), "DarkHyperlink")
				];
		}

		static void HandleHyperlinkClicked(const TWeakObjectPtr<UObject>& InObjectPtr)
		{
			if (InObjectPtr.IsValid())
			{
				AActor* Actor = StatsViewerUtils::GetActor( InObjectPtr );
				if( Actor )
				{
					const FScopedTransaction Transaction( LOCTEXT("SelectActors", "Statistics Select Actors") );
					GEditor->SelectNone(false, false);
					GEditor->SelectActor(Actor, true, true, true);
					GEditor->MoveViewportCamerasToActor(*Actor, false);
				}
				else
				{
					TArray< UObject* > ObjectsToSelect;
					ObjectsToSelect.Add( InObjectPtr.Get() );
					GEditor->SyncBrowserToObjects(ObjectsToSelect);
				}				
			}
		}

		static bool HandleIsClassSupported(const UClass* InClass)
		{
			if( InClass == UObject::StaticClass() || 
				InClass == AActor::StaticClass() || 
				InClass == UStaticMesh::StaticClass() ||
				InClass == UTexture::StaticClass() )
			{
				return true;
			}

			return false;
		}
	};

	if(!OnObjectHyperlinkClicked.IsBound())
	{
		OnObjectHyperlinkClicked = FOnObjectHyperlinkClicked::CreateStatic(&Local::HandleHyperlinkClicked);
	}

	if(!OnIsClassSupported.IsBound())
	{
		OnIsClassSupported = FOnIsClassSupported::CreateStatic(&Local::HandleIsClassSupported);
	}

	if(!OnGenerateWidget.IsBound())
	{
		OnGenerateWidget = FOnGenerateWidget::CreateStatic(&Local::HandleGenerateWidget, OnObjectHyperlinkClicked);
	}
}

bool FObjectHyperlinkColumn::Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const
{
	check(OnIsClassSupported.IsBound());

	if( Column->GetDataSource()->IsValid() )
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			const FProperty* Property = PropertyInfo.Property.Get();
			if( Property->IsA( FWeakObjectProperty::StaticClass() ) )
			{
				const UClass* PropertyClass = CastField<FWeakObjectProperty>(Property)->PropertyClass;
				if(OnIsClassSupported.Execute(PropertyClass))
				{
					return true;
				}
			}
		}
	}

	return false;
}

TSharedPtr< SWidget > FObjectHyperlinkColumn::CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	return NULL;
}

TSharedPtr< IPropertyTableCellPresenter > FObjectHyperlinkColumn::CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if( PropertyHandle.IsValid() )
	{
		UObject* Object = NULL;
		if( PropertyHandle->GetValue( Object ) == FPropertyAccess::Success )
		{
			return MakeShareable( new FObjectHyperlinkCellPresenter(Object, OnGenerateWidget) );
		}
	}

	return NULL;
}


#undef LOCTEXT_NAMESPACE
