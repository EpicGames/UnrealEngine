// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPropertyTableUtilities.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableCustomColumn.h"
#include "ObjectHyperlinkColumnInitializationOptions.h"

class IPropertyTableCell;
class IPropertyTableCellPresenter;

/**
 * A property table custom column used to display names of objects
 * that can be clicked on to jump to the objects in the scene or 
 * content browser.
 */
class FObjectHyperlinkColumn : public IPropertyTableCustomColumn
{
public:
	FObjectHyperlinkColumn(const FObjectHyperlinkColumnInitializationOptions& InOptions = FObjectHyperlinkColumnInitializationOptions());

	/** Begin IPropertyTableCustomColumn interface */
	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const override;
	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	/** End IPropertyTableCustomColumn interface */

private:

	/** Delegate called to generate a custom widget */
	FOnGenerateWidget OnGenerateWidget;

	/** Delegate called when a hyperlink is clicked */
	FOnObjectHyperlinkClicked OnObjectHyperlinkClicked;

	/** Delegate used to query whether a UClass is supported */
	FOnIsClassSupported OnIsClassSupported;
};

