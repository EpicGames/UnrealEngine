// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatsViewer.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorEngine.h"
#include "StatsViewerModule.h"
#include "PropertyEditorModule.h"
#include "IPropertyTableRow.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableCell.h"
#include "StatsPageManager.h"
#include "IPropertyTableCustomColumn.h"
#include "ObjectHyperlinkColumn.h"
#include "StatsCustomColumn.h"
#include "ActorArrayHyperlinkColumn.h"
#include "PropertyPath.h"
#include "StatsViewerUtils.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

DEFINE_LOG_CATEGORY_STATIC(LogStatsViewer, Log, All);

#define LOCTEXT_NAMESPACE "Editor.StatsViewer"

namespace StatsViewerConstants
{
	/** Delay (in seconds) after a new character is entered into the search box to wait before updating the list (to give them time to enter a whole string instead of useless updating every time a char is put in) **/
	static const float SearchTextUpdateDelay = 0.5f;

	/** Stat viewer config file section name */
	static const FString ConfigSectionName = "StatsViewer";
}

namespace StatsViewerMetadata
{
	static const FName ColumnWidth( "ColumnWidth" );
	static const FName SortMode( "SortMode" );
}

void SStatsViewer::Construct( const FArguments& InArgs )
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	StatsPageManagerPtr = InArgs._StatsPageManager;

	// create empty property table
	PropertyTable = PropertyEditorModule.CreatePropertyTable();
	PropertyTable->SetIsUserAllowedToChangeRoot( false );
	PropertyTable->SetOrientation( EPropertyTableOrientation::AlignPropertiesInColumns );
	PropertyTable->SetShowRowHeader( false );
	PropertyTable->SetShowObjectName( false );

	// we want to customize some columns
	TArray< TSharedRef< IPropertyTableCustomColumn > > CustomColumns;
	FStatsPageManager& StatsPageManager = GetStatsPageManager();
	for( int32 PageIndex = 0; PageIndex < StatsPageManager.NumPages(); PageIndex++ )
	{
		TSharedRef<IStatsPage> StatsPage = StatsPageManager.GetPageByIndex( PageIndex );
		TArray< TSharedRef< IPropertyTableCustomColumn > > PagesCustomColumns;
		StatsPage->GetCustomColumns(PagesCustomColumns);
		if(PagesCustomColumns.Num() > 0)
		{
			CustomColumns.Append(PagesCustomColumns);
		}
	}

	CustomColumns.Add( MakeShareable(new FObjectHyperlinkColumn) );
	CustomColumns.Add( MakeShareable(new FActorArrayHyperlinkColumn) );
	CustomColumns.Add( CustomColumn );

	ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
		.AutoHeight()
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Padding(4.0f)
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f)
				[
					SNew( SComboButton )
					.ContentPadding(3.f)
					.OnGetMenuContent( this, &SStatsViewer::OnGetDisplayMenuContent )
					.ButtonContent()
					[
						SNew( STextBlock )
						.Text( this, &SStatsViewer::OnGetDisplayMenuLabel )
						.ToolTipText( LOCTEXT( "DisplayedStatistic_Tooltip", "Choose the statistics to display" ) )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f)
				[
					SNew( SButton )
					.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
					.ContentPadding(3.f)
					.OnClicked( this, &SStatsViewer::OnRefreshClicked )
					.Content()
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "Refresh", "Refresh" ) )
						.ToolTipText( LOCTEXT( "Refresh_Tooltip", "Refresh the displayed statistics" ) )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f)
				[
					SNew( SButton )
					.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
					.ContentPadding(3.f)
					.OnClicked( this, &SStatsViewer::OnExportClicked )
					.Content()
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "Export", "Export" ) )
						.ToolTipText( LOCTEXT( "Export_Tooltip", "Export the displayed statistics to a CSV file" ) )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f)
				[
					SAssignNew( CustomContent, SBorder )
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
					.Padding(0.0f)
					.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f)
				.HAlign(HAlign_Right)
				[
					SAssignNew( CustomFilter, SBorder )
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
					.Padding(0.0f)
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
		.FillHeight(1.0f)
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
			.Padding(4.0f)
			[
				PropertyEditorModule.CreatePropertyTableWidget( PropertyTable.ToSharedRef(), CustomColumns )
			]
		]
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
		.AutoHeight()
		[	
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
			.Padding(4.0f)
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew( FilterTextBoxWidget, SSearchBox )
					.HintText( LOCTEXT( "FilterDisplayedStatistics", "Filter Displayed Statistics" ) )
					.ToolTipText( LOCTEXT( "FilterDisplayedStatistics_Tooltip", "Type here to filter displayed statistics" ) )
					.OnTextChanged( this, &SStatsViewer::OnFilterTextChanged )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 1, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew( SComboButton )
					.Visibility( this, &SStatsViewer::OnGetStatsVisibility )
					.ContentPadding(2.f)
					.OnGetMenuContent( this, &SStatsViewer::OnGetFilterMenuContent )
					.ButtonContent()
					[
						SNew( STextBlock )
						.Text( this, &SStatsViewer::OnGetFilterComboButtonLabel )
						.ToolTipText( LOCTEXT( "FilterColumnToUse_Tooltip", "Choose the statistic to filter when searching" ) )
					]
				]
			]
		]
	];

	// Display stats page from previous stat viewer instance
	if (!CurrentStats.IsValid())
	{
		TSharedPtr<IStatsPage> InitialStatsPage;
		FString DisplayedStatsPageName;
		FString ConfigKey = TEXT("DisplayedStatsPageName");
		if (StatsPageManager.GetName() != NAME_None)
		{
			ConfigKey += TEXT("_") + StatsPageManager.GetName().ToString();
		}

		if (GConfig->GetString(*StatsViewerConstants::ConfigSectionName, *ConfigKey, DisplayedStatsPageName, GEditorPerProjectIni))
		{
			InitialStatsPage = StatsPageManager.GetPage(FName(*DisplayedStatsPageName));
		}

		if(!InitialStatsPage.IsValid())
		{
			// Default to primitive stats if no config data exists yet
			InitialStatsPage = StatsPageManager.GetPage(EStatsPage::PrimitiveStats);
		}

		SetDisplayedStats(InitialStatsPage.ToSharedRef());
	}
}

SStatsViewer::SStatsViewer() :
	bNeedsRefresh( false ),
	bNeedsRefreshForFilterChange( false ),
	CurrentObjectSetIndex( 0 ),
	CurrentFilterIndex( 0 ),
	CustomColumn( new FStatsCustomColumn )
{
}

SStatsViewer::~SStatsViewer()
{
	if( CurrentStats.IsValid() )
	{
		CurrentStats->OnHide();
	}
}

/** Helper function to get the string of a cell as it is being presented to the user */
static FString GetCellString( const TSharedPtr<IPropertyTableCell> Cell, bool bGetRawValue = false )
{
	FString String = TEXT("");

	// we dont want to search the full object path if this is an object, so
	// we use the displayed name we would get from our asset hyperlink column
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if( PropertyHandle.IsValid() )
	{
		UObject* Object = NULL;
		if( PropertyHandle->GetValue( Object ) == FPropertyAccess::Success )
		{
			if( Object != NULL )
			{
				String = StatsViewerUtils::GetAssetName( Object ).ToString();
			}
		}
	}

	// not an object, but maybe supported
	if( FStatsCustomColumn::SupportsProperty(PropertyHandle->GetProperty()) )
	{
		String = FStatsCustomColumn::GetPropertyAsText(PropertyHandle, bGetRawValue).ToString();
	}

	// still no name? will have to default to the 'value as string'
	if(String.Len() == 0)
	{
		String = Cell->GetValueAsString();
	}

	return String;
}

void SStatsViewer::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// check if we need to switch pages - i.e. if a page wants to be shown
	FStatsPageManager& StatsPageManager = GetStatsPageManager();
	for( int32 PageIndex = 0; PageIndex < StatsPageManager.NumPages(); PageIndex++ )
	{
		TSharedRef<IStatsPage> StatsPage = StatsPageManager.GetPageByIndex( PageIndex );
		if( StatsPage->IsShowPending() )
		{
			SetDisplayedStats( StatsPage );
			StatsPage->Show( false );
		}
	}

	// check if we have timed out after typing something into the search filter
	bool bTimerActive = SearchTextUpdateTimer >= 0.0f;
	SearchTextUpdateTimer -= InDeltaTime;
	if( bTimerActive && SearchTextUpdateTimer < 0.0f )
	{
		bNeedsRefreshForFilterChange = true;
	}

	if( CurrentStats.IsValid() )
	{
		if( CurrentStats->IsRefreshPending() )
		{
			bNeedsRefresh = true;
			CurrentStats->Refresh( false );
		}
	}

	if( bNeedsRefresh || bNeedsRefreshForFilterChange)
	{
		if( CurrentStats.IsValid() )
		{
			if (bNeedsRefresh)
			{
				// Flag all the current stat objects for death
				for (auto Iter = LastGeneratedObjectList.CreateIterator(); Iter; Iter++)
				{
					if ((*Iter).IsValid())
					{
						(*Iter)->RemoveFromRoot();
					}
				}

				// Generate new set of objects
				LastGeneratedObjectList.Empty();
				CurrentObjects.Empty();
				CurrentStats->Generate(CurrentObjects);

				// Backup list for future use (see bNeedsRefreshForFilterChange)
				LastGeneratedObjectList = CurrentObjects;
			}
			else if (bNeedsRefreshForFilterChange)
			{
				// For a filter change, recycle last generated object list
				CurrentObjects = LastGeneratedObjectList;
			}

			// clear the map of total strings
			CustomColumn->TotalsMap.Empty();

			// plug objects into table
			PropertyTable->SetObjects(CurrentObjects);

			// freeze & resize columns & sort if required
			const TArray< TSharedRef< IPropertyTableColumn > >& Columns = PropertyTable->GetColumns();
			for (int32 ColumnIndex = 0; ColumnIndex < Columns.Num(); ++ColumnIndex)
			{
				TSharedRef< IPropertyTableColumn > Column = Columns[ColumnIndex];
				if (Columns[ColumnIndex]->GetDataSource()->IsValid())
				{
					TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
					const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
					const FString& ColumnWidthString = PropertyInfo.Property->GetMetaData(StatsViewerMetadata::ColumnWidth);
					const float ColumnWidth = ColumnWidthString.Len() > 0 ? FCString::Atof(*ColumnWidthString) : 100.0f;
					Column->SetWidth(ColumnWidth);

					const FString& SortModeString = PropertyInfo.Property->GetMetaData(StatsViewerMetadata::SortMode);
					if (SortModeString.Len() > 0)
					{
						EColumnSortMode::Type SortType = SortModeString == TEXT("Ascending") ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
						PropertyTable->SortByColumn(Column, SortType, EColumnSortPriority::Primary);
					}
				}

				Column->SetFrozen(true);
			}

			// Cull objects using filter - this is currently a bit of a hack, as we need to modify the source data
			// rather than the view of that data (i.e. the property table).
			// @todo: Fix this once the property table has filtering.
			if(FilterText.Len() > 0)
			{
				TArray< TSharedRef< IPropertyTableRow > >& Rows = PropertyTable->GetRows();
				for( int32 RowIndex = 0; RowIndex < Rows.Num(); RowIndex++ )
				{
					bool bFoundMatchingCell = false;

					int32 ColumnIndex = 0;
					for( TSharedPtr< IPropertyTableCell > Cell = PropertyTable->GetFirstCellInRow(Rows[RowIndex]); Cell.IsValid(); Cell = PropertyTable->GetNextCellInRow(Cell.ToSharedRef()), ColumnIndex++ )
					{
						if( CurrentFilterIndex == ColumnIndex )
						{
							FString String = GetCellString( Cell );
							
							if( String.Contains(FilterText) )
							{
								bFoundMatchingCell = true;
							}
							break;
						}
					}

					if( !bFoundMatchingCell )
					{
						CurrentObjects.Remove( Rows[RowIndex]->GetDataSource()->AsUObject() );
					}
				}
			}

			// generate totals from the objects that are filtered
			CurrentStats->GenerateTotals( CurrentObjects, CustomColumn->TotalsMap );

			// Re-plug objects into table as we may have removed some 
			// note: this currently also allows us to properly set up the UI as the 'frozen'
			// flag is not taken into account when building the table at first. Setting the
			// same set of objects again here allows us to remove the combo button & 'remove' menu
			// from the column header.
			PropertyTable->SetObjects( CurrentObjects );

			PropertyTable->RequestRefresh();
		}

		bNeedsRefresh = false;
		bNeedsRefreshForFilterChange = false;
	}
}

void SStatsViewer::Refresh()
{
	bNeedsRefresh = true;
}

TSharedPtr< IPropertyTable > SStatsViewer::GetPropertyTable()
{
	return PropertyTable;
}

FReply SStatsViewer::OnRefreshClicked()
{
	Refresh();

	return FReply::Handled();
}

FReply SStatsViewer::OnExportClicked()
{
	if( !CurrentStats.IsValid() || CurrentObjects.Num() == 0 )
	{
		return FReply::Handled();
	}

	// MEssage to disply on completion
	FText Message;
	bool bSuccessful = false;

	// CSV: Human-readable spreadsheet format.
	FString CSVFilename = FPaths::ProjectLogDir();
	CSVFilename /= CurrentStats->GetName().ToString();
	CSVFilename /= GWorld->GetOutermost()->GetName();
	CSVFilename /= FString::Printf(
		TEXT("%s-%i-%s.csv"),
		FApp::GetProjectName(),
		FEngineVersion::Current().GetChangelist(),
		*FDateTime::Now().ToString() );

	// Create the CSV (can't use ToNumber or FormatIntToHumanReadable as it'll break the CSV!)
	FArchive* CSVFile = IFileManager::Get().CreateFileWriter( *CSVFilename );
	if( CSVFile )
	{
		const FString Delimiter = TEXT(",");
		
		// write out header row
		{
			FString HeaderRow;
			const TArray< TSharedRef< IPropertyTableColumn > >& Columns = PropertyTable->GetColumns();
			for( int32 ColumnIndex = 0; ColumnIndex < Columns.Num(); ++ColumnIndex )
			{
				TSharedRef< IPropertyTableColumn > Column = Columns[ColumnIndex];
				if( Columns[ColumnIndex]->GetDataSource()->IsValid() )
				{
					TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
					const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
					const TWeakFieldPtr< const FProperty > Property = PropertyInfo.Property;
					FString Name = UEditorEngine::GetFriendlyName(Property.Get());
					Name.ReplaceInline( *Delimiter, TEXT(" ") );
					HeaderRow += Name + Delimiter;
				}
			}
			HeaderRow += LINE_TERMINATOR;
			CSVFile->Serialize( TCHAR_TO_ANSI( *HeaderRow ), HeaderRow.Len() );
		}
			
		// write out data
		{
			FString Data;
			TArray< TSharedRef< IPropertyTableRow > >& Rows = PropertyTable->GetRows();
			for( int32 RowIndex = 0; RowIndex < Rows.Num(); ++RowIndex )
			{
				TSharedRef< IPropertyTableRow > Row = Rows[RowIndex];
				for( TSharedPtr< IPropertyTableCell > Cell = PropertyTable->GetFirstCellInRow(Row); Cell.IsValid(); Cell = PropertyTable->GetNextCellInRow(Cell.ToSharedRef()) )
				{
					FString CellData = GetCellString( Cell , true );
					CellData.ReplaceInline( *Delimiter, TEXT(" ") );
					Data += CellData + Delimiter;
				}
				Data += LINE_TERMINATOR;
			}
			CSVFile->Serialize( TCHAR_TO_ANSI( *Data ), Data.Len() );
		}

		// write out totals, if any
		{
			FString Total;
			const TArray< TSharedRef< IPropertyTableColumn > >& Columns = PropertyTable->GetColumns();
			for( int32 ColumnIndex = 0; ColumnIndex < Columns.Num(); ++ColumnIndex )
			{
				TSharedRef< IPropertyTableColumn > Column = Columns[ColumnIndex];
				if( Columns[ColumnIndex]->GetDataSource()->IsValid() )
				{
					TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
					const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
					const FString& ShowTotal = PropertyInfo.Property->GetMetaData(TEXT("ShowTotal"));
					if( ShowTotal.Len() > 0 )
					{
						FText* TotalText = CustomColumn->TotalsMap.Find( PropertyInfo.Property->GetNameCPP() );
						if( TotalText != NULL )
						{
							FString TotalString = TotalText->ToString();
							TotalString.ReplaceInline( *Delimiter, TEXT(" ") );
							Total += TotalString;
						}							
					}
				}

				Total += Delimiter;
			}
			Total += LINE_TERMINATOR;
			CSVFile->Serialize( TCHAR_TO_ANSI( *Total ), Total.Len() );
		}

		// Close and delete archive.
		CSVFile->Close();
		delete CSVFile;
		CSVFile = NULL;

		Message = LOCTEXT("ExportMessage", "Wrote statistics to file");
		bSuccessful = true;
	}
	else
	{
		Message = LOCTEXT("ExportErrorMessage", "Could not write statistics to file");
		bSuccessful = false;
	}

	struct Local
	{
		static void NavigateToExportedFile( FString InCSVFilename, bool bInSuccessful )
		{
			InCSVFilename = FPaths::ConvertRelativePathToFull(InCSVFilename);
			if (bInSuccessful)
			{
				FPlatformProcess::LaunchFileInDefaultExternalApplication(*InCSVFilename);
			}
			else
			{
				FPlatformProcess::ExploreFolder(*FPaths::GetPath(InCSVFilename));
			}
		}
	};

	FNotificationInfo Info(Message);
	Info.Hyperlink = FSimpleDelegate::CreateStatic(&Local::NavigateToExportedFile, CSVFilename, false);
	Info.HyperlinkText = FText::FromString( CSVFilename );
	Info.bUseLargeFont = false;
	Info.bFireAndForget = true;
	Info.ExpireDuration = 8.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF( LogStatsViewer, Log, "%ls %ls", *Message.ToString(), *CSVFilename );

	return FReply::Handled();
}

FStatsPageManager& SStatsViewer::GetStatsPageManager() const
{
	// We can use either provided page manager or use the default (global) one.
	return StatsPageManagerPtr.IsValid() ? *StatsPageManagerPtr : FStatsPageManager::Get();
}

int32 SStatsViewer::GetObjectSetIndex() const
{
	return CurrentObjectSetIndex;
}

void SStatsViewer::SwitchAndFilterPage(int32 Page, const FString& FilterValue, const FString& FilterProperty)
{
	FStatsPageManager& Manager = FStatsPageManager::Get();
	if(Page < Manager.NumPages())
	{
		TSharedRef<IStatsPage> StatsPage = FStatsPageManager::Get().GetPage(Page);
		SetDisplayedStats(StatsPage);

		int32 FilterIndex = -1;
		if(FilterProperty.Len())
		{
			int32 ColumnIndex = 0;
			for (TFieldIterator<FProperty> PropertyIter( CurrentStats->GetEntryClass(), EFieldIteratorFlags::IncludeSuper ); PropertyIter; ++PropertyIter )
			{
				FProperty* Property = *PropertyIter;
				if( Property->HasAnyPropertyFlags(CPF_AssetRegistrySearchable) )
				{
					FString FilterName = Property->GetDisplayNameText().ToString();
					if( FilterName.Len() == 0 )
					{
						FilterName = UEditorEngine::GetFriendlyName(Property);
					}
					if(FilterName == FilterProperty)
					{
						FilterIndex = ColumnIndex;
						break;
					}
					ColumnIndex++;
				}
			}
		}
		if(FilterIndex >= 0)
		{
			FilterTextBoxWidget->SetText(FText::FromString(FilterValue));
			SetSearchFilter(FilterIndex);
		}
		Refresh();
	}
}
void SStatsViewer::OnFilterTextChanged( const FText& InFilterText )
{
	FilterText = InFilterText.ToString();
	SearchTextUpdateTimer = StatsViewerConstants::SearchTextUpdateDelay;
}

FText SStatsViewer::OnGetDisplayMenuLabel() const
{
	if( CurrentStats.IsValid() )
	{
		return CurrentStats->GetDisplayName();
	}

	return LOCTEXT( "NoDisplayedStatistic", "Display" );
}

FText SStatsViewer::OnGetObjectSetMenuLabel() const
{
	if( CurrentStats.IsValid() )
	{
		return FText::FromString( CurrentStats->GetObjectSetName(CurrentObjectSetIndex) );
	}

	return LOCTEXT( "NoDisplayedObjectSet", "Objects" );
}

TSharedRef<SWidget> SStatsViewer::OnGetDisplayMenuContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);
	FStatsPageManager& StatsPageManager = GetStatsPageManager();

	for( int32 PageIndex = 0; PageIndex < StatsPageManager.NumPages(); PageIndex++ )
	{
		TSharedRef<IStatsPage> StatsPage = StatsPageManager.GetPageByIndex( PageIndex );
		MenuBuilder.AddMenuEntry( 
			StatsPage->GetDisplayName(), 
			StatsPage->GetToolTip(), 
			FSlateIcon(), 
			FUIAction( 
				FExecuteAction::CreateSP( const_cast<SStatsViewer*>(this), &SStatsViewer::SetDisplayedStats, StatsPage ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SStatsViewer::AreStatsDisplayed, StatsPage )
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton 
		);
	}
	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SStatsViewer::OnGetObjectSetMenuContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	if( CurrentStats.IsValid() )
	{
		for( int32 ObjectSetIndex = 0; ObjectSetIndex < CurrentStats->GetObjectSetCount(); ++ObjectSetIndex )
		{
			MenuBuilder.AddMenuEntry( 
				FText::FromString( CurrentStats->GetObjectSetName( ObjectSetIndex ) ), 
				FText::FromString( CurrentStats->GetObjectSetToolTip( ObjectSetIndex ) ), 
				FSlateIcon(), 
				FUIAction( 
					FExecuteAction::CreateSP( const_cast<SStatsViewer*>(this), &SStatsViewer::SetObjectSet, ObjectSetIndex ),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP( this, &SStatsViewer::IsObjectSetSelected, ObjectSetIndex )
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton 
			);
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SStatsViewer::OnGetFilterMenuContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	if( CurrentStats.IsValid() )
	{
		int32 ColumnIndex = 0;
		for (TFieldIterator<FProperty> PropertyIter( CurrentStats->GetEntryClass(), EFieldIteratorFlags::IncludeSuper ); PropertyIter; ++PropertyIter )
		{
			FProperty* Property = *PropertyIter;
			if( Property->HasAnyPropertyFlags(CPF_AssetRegistrySearchable) )
			{
				FText FilterName = Property->GetDisplayNameText();
				if( FilterName.IsEmpty() )
				{
					FilterName = FText::AsCultureInvariant(UEditorEngine::GetFriendlyName(Property));
				}

				FText FilterDesc = Property->GetToolTipText();
				if( FilterDesc.IsEmpty() )
				{
					FilterDesc = FText::AsCultureInvariant(UEditorEngine::GetFriendlyName(Property));
				}

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("FilterName"), FilterName);
				Arguments.Add(TEXT("FilterDesc"), FilterDesc);

				MenuBuilder.AddMenuEntry( 
					FilterName, 
					FText::Format( LOCTEXT( "FilterMenuEntry_Tooltip", "Search statistics by {FilterName}.\n{FilterDesc}" ), Arguments ), 
					FSlateIcon(), 
					FUIAction( 
						FExecuteAction::CreateSP( const_cast<SStatsViewer*>(this), &SStatsViewer::SetSearchFilter, ColumnIndex ),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP( this, &SStatsViewer::IsSearchFilterSelected, ColumnIndex )
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton 
				);

				++ColumnIndex;
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

EVisibility SStatsViewer::OnGetObjectSetsVisibility() const
{
	if( CurrentStats.IsValid() )
	{
		return CurrentStats->GetObjectSetCount() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return CurrentStats.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SStatsViewer::OnGetStatsVisibility() const
{
	return CurrentStats.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SStatsViewer::SetDisplayedStats( TSharedRef<IStatsPage> StatsPage )
{
	if( CurrentStats.IsValid() )
	{
		CurrentStats->OnHide();
	}

	CurrentStats = StatsPage;
	GConfig->SetString(*StatsViewerConstants::ConfigSectionName, TEXT("DisplayedStatsPageName"), *StatsPage->GetName().ToString(), GEditorPerProjectIni);
	CurrentStats->OnShow( SharedThis(this) );
	CurrentObjectSetIndex = CurrentStats->GetSelectedObjectSet();
	CurrentFilterIndex = 0;
	FilterTextBoxWidget->SetText( FText::FromString(TEXT("")) );

	// set custom widget, if any
	TSharedPtr<SWidget> CustomContentWidget = CurrentStats->GetCustomWidget( SharedThis(this) );
	{
		if(CustomContentWidget.IsValid())
		{
			CustomContent->SetContent( CustomContentWidget.ToSharedRef() );
			CustomContent->SetVisibility( EVisibility::Visible );
		}
		else
		{
			CustomContent->SetVisibility( EVisibility::Collapsed );
		}
	}

	// set custom filter, if any
	TSharedPtr<SWidget> CustomFilterWidget = CurrentStats->GetCustomFilter( SharedThis(this) );
	{
		if (!CustomFilterWidget.IsValid())
		{
			CustomFilterWidget = SNew( SComboButton )
				.Visibility( this, &SStatsViewer::OnGetObjectSetsVisibility )
				.ContentPadding(3.f)
				.OnGetMenuContent( this, &SStatsViewer::OnGetObjectSetMenuContent )
				.ButtonContent()
				[
					SNew( STextBlock )
						.Text( this, &SStatsViewer::OnGetObjectSetMenuLabel )
						.ToolTipText( LOCTEXT( "DisplayedObjects_Tooltip", "Choose the objects whose statistics you want to display" ) )
				];
		}

		CustomFilter->SetContent(CustomFilterWidget.ToSharedRef());
	}

	Refresh();
}

bool SStatsViewer::AreStatsDisplayed( TSharedRef<IStatsPage> StatsPage ) const
{
	return CurrentStats == StatsPage;
}

FText SStatsViewer::OnGetFilterComboButtonLabel() const
{
	if( CurrentStats.IsValid() )
	{
		int32 ColumnIndex = 0;
		for (TFieldIterator<FProperty> PropertyIter( CurrentStats->GetEntryClass(), EFieldIteratorFlags::IncludeSuper ); PropertyIter; ++PropertyIter )
		{
			if( PropertyIter->HasAnyPropertyFlags(CPF_AssetRegistrySearchable) )
			{
				if( ColumnIndex == CurrentFilterIndex )
				{
					FString FilterName = PropertyIter->GetDisplayNameText().ToString();
					if( FilterName.Len() == 0 )
					{
						FilterName = UEditorEngine::GetFriendlyName(*PropertyIter);
					}

					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("FilterName"), FText::FromString(FilterName));
					return FText::Format( LOCTEXT( "FilterSelected", "Filter: {FilterName}" ), Arguments );
				}
				++ColumnIndex;
			}
		}
	}

	return LOCTEXT( "Filter", "Filter" );
}

void SStatsViewer::SetObjectSet( int32 InSetIndex )
{
	CurrentObjectSetIndex = InSetIndex;
	if( CurrentStats.IsValid() )
	{
		CurrentStats->SetSelectedObjectSet( InSetIndex );
	}

	Refresh();
}

bool SStatsViewer::IsObjectSetSelected( int32 InSetIndex ) const
{
	return CurrentObjectSetIndex == InSetIndex;
}

void SStatsViewer::SetSearchFilter( int32 InFilterIndex )
{
	CurrentFilterIndex = InFilterIndex;

	bNeedsRefreshForFilterChange = true;
}

bool SStatsViewer::IsSearchFilterSelected( int32 InFilterIndex ) const
{
	return CurrentFilterIndex == InFilterIndex;
}

#undef LOCTEXT_NAMESPACE
