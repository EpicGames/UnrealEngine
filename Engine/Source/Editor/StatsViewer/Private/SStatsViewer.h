// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPropertyTable.h"
#include "IStatsViewer.h"
#include "IStatsPage.h"

class FStatsPageManager;

/**
 * Stats Viewer widget
 */
class SStatsViewer : public IStatsViewer
{
public:

	SLATE_BEGIN_ARGS( SStatsViewer ){}
	SLATE_ARGUMENT(TSharedPtr< FStatsPageManager >, StatsPageManager)
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs		Declaration used by the SNew() macro to construct this widget
	 */
	void Construct( const FArguments& InArgs );

	/** SStatsViewer constructor */
	SStatsViewer();

	/** SStatsViewer destructor */
	virtual ~SStatsViewer();

	/** SWidget interface */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Begin IStatsViewer interface */
	virtual void Refresh() override;
	TSharedPtr< class IPropertyTable > GetPropertyTable() override;
	int32 GetObjectSetIndex() const override;
	void SwitchAndFilterPage(int32 Page, const FString& FilterValue, const FString& FilterProperty) override;
	/** End IStatsViewer interface */

private:

	/**
	 * Called by the editable text control when the filter text is changed by the user
	 *
	 * @param	InFilterText	The new text
	 */
	void OnFilterTextChanged( const FText& InFilterText );

	/** 
	 * Called by the editable text control when a user presses enter or commits their text change 
	 * @param	InFilterText	The new text
	 * @param	CommitInfo		The type of commit
	 */
	void OnFilterTextCommitted( const FText& InFilterText, ETextCommit::Type CommitInfo );

	/** Generate label for the display combobox */
	FText OnGetDisplayMenuLabel() const;

	/** Generate label for the displayed objects combobox */
	FText OnGetObjectSetMenuLabel() const;

	/** Build the menu for the display combobox */
	TSharedRef<SWidget> OnGetDisplayMenuContent() const;

	/** Build the menu for the object set combobox */
	TSharedRef<SWidget> OnGetObjectSetMenuContent() const;

	/** Build the menu for the filter combobox */
	TSharedRef<SWidget> OnGetFilterMenuContent() const;

	/** Check whether the object sets combo box is visible for the currently selected stats */
	EVisibility OnGetObjectSetsVisibility() const;

	/** Check whether the various UI elements are visible for the currently selected stats */
	EVisibility OnGetStatsVisibility() const;

	/** 
	 * Set the displayed set of stats 
	 * @param	StatsPage The stats page to set
	 */
	void SetDisplayedStats( TSharedRef< class IStatsPage > StatsPage );

	/** 
	 * Check if the displayed set of stats is currently visible
	 * @param	StatsPage The stats page to set
	 * @returns true if the page is currently visible
	 */
	bool AreStatsDisplayed( TSharedRef< class IStatsPage > StatsPage ) const;

	/** Get the label for the filter combo button */
	FText OnGetFilterComboButtonLabel() const;

	/** 
	 * Set the currently displayed object set 
	 * @param	InSetIndex	The index of the object set to display
	 */
	void SetObjectSet( int32 InSetIndex );

	/** 
	 * Check whether the currently displayed object set matches the passed-in index
	 * @param	InSetIndex	The index of the object set to check
	 * @returns true if the displayed object set matches the passed in index
	 */
	bool IsObjectSetSelected( int32 InSetIndex ) const;

	/** 
	 * Set the currently selected search filter 
	 * @param	InFilterIndex	The filter index to set
	 */
	void SetSearchFilter( int32 InFilterIndex );

	/** 
	 * Check whether the currently displayed search filter matches the passed-in index
	 * @param	InFilterIndex	The index of the search filter to check
	 * @returns true if the displayed search filter matches the passed in index
	 */
	bool IsSearchFilterSelected( int32 InFilterIndex ) const;

	/** We just clicked the refresh button */
	FReply OnRefreshClicked();

	/** We just clicked the export button */
	FReply OnExportClicked();

	/** Get a valid stats page manager */
	FStatsPageManager& GetStatsPageManager() const;

private:

	/** Flag to refresh the table next tick */
	bool bNeedsRefresh;

	/** Flag to refresh the table triggered by a filter change */
	bool bNeedsRefreshForFilterChange;

	/** Timer to prevent constant update of the searched items when typing */
	float SearchTextUpdateTimer;

	/* Widget containing the filtering text box */
	TSharedPtr< SSearchBox > FilterTextBoxWidget;

	/** The property table we are viewing */
	TSharedPtr< class IPropertyTable > PropertyTable;

	/* The currently displayed stats factory */
	TSharedPtr< class IStatsPage > CurrentStats;

	/** The current object set selected */
	int32 CurrentObjectSetIndex;

	/** The current filter text */
	FString FilterText;

	/** The current filter selected */
	int32 CurrentFilterIndex;

	/** The current set of objects we are viewing */
	TArray< TWeakObjectPtr<UObject> > CurrentObjects;

	/** The last generated set of objects that will be used when refresh triggered by a filter change */
	TArray< TWeakObjectPtr<UObject> > LastGeneratedObjectList;

	/** The 'total' custom column used for displaying totals for properties that support the feature */
	TSharedRef<class FStatsCustomColumn> CustomColumn;

	/** Container for custom content supplied by stats pages */
	TSharedPtr< SBorder > CustomContent;

	/** Container for custom filters supplied by stats pages */
	TSharedPtr< SBorder > CustomFilter;

	/** The page manager that is managing our pages. Can be invalid, in which case the global one will be used */
	TSharedPtr < FStatsPageManager > StatsPageManagerPtr;
};

