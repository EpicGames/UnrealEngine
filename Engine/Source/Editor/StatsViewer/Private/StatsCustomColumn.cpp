// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsCustomColumn.h"

#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformMath.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "SlotBase.h"
#include "StatsCellPresenter.h"
#include "Styling/AppStyle.h"
#include "Templates/TypeHash.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyTableUtilities;
class SWidget;

class FNumericStatCellPresenter : public TSharedFromThis< FNumericStatCellPresenter >, public FStatsCellPresenter
{
public:

	FNumericStatCellPresenter( const TSharedPtr< IPropertyHandle > PropertyHandle )
	{
		Text = FStatsCustomColumn::GetPropertyAsText( PropertyHandle );
	}

	virtual ~FNumericStatCellPresenter() {}

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override
	{
		return 
			SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( STextBlock )
					.Text( Text )
				];
	}
};

bool FStatsCustomColumn::Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const
{
	if( Column->GetDataSource()->IsValid() )
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if( PropertyPath->GetNumProperties() > 0 )
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			return FStatsCustomColumn::SupportsProperty( PropertyInfo.Property.Get() );
		}
	}

	return false;
}

TSharedPtr< SWidget > FStatsCustomColumn::CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
	const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
	if( PropertyInfo.Property.Get()->HasMetaData( "ShowTotal" ) )
	{
		return 
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( STextBlock )
				.Font( FAppStyle::GetFontStyle( Style ) )
				.Text( Column->GetDisplayName() )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( STextBlock )
				.Font( FAppStyle::GetFontStyle(TEXT("BoldFont") ) )
				.Text( this, &FStatsCustomColumn::GetTotalText, Column )
			];
	}
	else
	{
		return 
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( STextBlock )
				.Font( FAppStyle::GetFontStyle( Style ) )
				.Text( Column->GetDisplayName() )
			];
	}
}

TSharedPtr< IPropertyTableCellPresenter > FStatsCustomColumn::CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if( PropertyHandle.IsValid() )
	{
		return MakeShareable( new FNumericStatCellPresenter( PropertyHandle ) );
	}

	return NULL;
}

FText FStatsCustomColumn::GetTotalText( TSharedRef< IPropertyTableColumn > Column ) const
{
	TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
	const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
	const FString PropertyName = PropertyInfo.Property->GetNameCPP();
	const FText* TotalText = TotalsMap.Find( PropertyName );

	if( TotalText != NULL )
	{
		FText OutText = *TotalText;
		if( PropertyInfo.Property.Get()->HasMetaData("Unit") )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT("Value"), OutText );
			Args.Add( TEXT("Unit"), FText::FromString( PropertyInfo.Property.Get()->GetMetaData("Unit") ) );
			OutText = FText::Format( NSLOCTEXT("Stats", "Value + Unit", "{Value} {Unit}"), Args );
		}

		return OutText;
	}

	return FText::GetEmpty();
}

bool FStatsCustomColumn::SupportsProperty( const FProperty* Property )
{
	if( Property->IsA( FFloatProperty::StaticClass() ) || Property->IsA( FIntProperty::StaticClass() ) )
	{
		return true;
	}

	if( Property->IsA( FStructProperty::StaticClass() ) )
	{
		return ( CastField<const FStructProperty>(Property)->Struct->GetFName() == NAME_Vector2D );
	}

	return false;
}

FText FStatsCustomColumn::GetPropertyAsText( const TSharedPtr< IPropertyHandle > PropertyHandle , bool bGetRawValue )
{
	FText Text;

	//Create a formatting option that doesn't group digits for readability. This will generate pure number strings.
	FNumberFormattingOptions RawFormattingOptions;
	FNumberFormattingOptions *RFOPointer = nullptr;

	RawFormattingOptions.SetUseGrouping(false);

	//Use ungrouped formatting option if requested by user. Leaving the pointer NULL will trigger usage of Locale default settings.
	if (bGetRawValue)
	{
		RFOPointer = &RawFormattingOptions;
	}


	if( PropertyHandle->GetProperty()->IsA( FIntProperty::StaticClass() ) )
	{
		int32 IntValue = INT_MAX;
		PropertyHandle->GetValue( IntValue );
		if( IntValue == INT_MAX )
		{
			Text = NSLOCTEXT("Stats", "UnknownIntegerValue", "?");
		}
		else
		{
			Text = FText::AsNumber( IntValue, RFOPointer);
		}
	}
	else if( PropertyHandle->GetProperty()->IsA( FFloatProperty::StaticClass() ) )
	{
		float FloatValue = FLT_MAX;
		PropertyHandle->GetValue( FloatValue );
		if( FloatValue == FLT_MAX )
		{
			Text = NSLOCTEXT("Stats", "UnknownFloatValue", "?");
		}
		else
		{
			Text = FText::AsNumber( FloatValue, RFOPointer);
		}
	}
	else if( PropertyHandle->GetProperty()->IsA( FStructProperty::StaticClass() ) )
	{
		if( CastField<const FStructProperty>(PropertyHandle->GetProperty())->Struct->GetFName() == NAME_Vector2D )
		{
			FVector2D VectorValue(0.0f, 0.0f);
			PropertyHandle->GetValue( VectorValue );
			if( VectorValue.X == FLT_MAX || VectorValue.Y == FLT_MAX )
			{
				Text = NSLOCTEXT("Stats", "UnknownVectorValue", "?");
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("VectorX"), VectorValue.X );
				Args.Add( TEXT("VectorY"), VectorValue.Y );
				Text = FText::Format( NSLOCTEXT("Stats", "VectorValue", "{VectorX}x{VectorY}"), Args );
			}
		}
	}

	if (PropertyHandle->GetProperty()->HasMetaData("Unit") && !bGetRawValue)
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("Value"), Text );
		Args.Add( TEXT("Unit"), FText::FromString( PropertyHandle->GetProperty()->GetMetaData("Unit") ) );
		Text = FText::Format( NSLOCTEXT("Stats", "Value + Unit", "{Value} {Unit}"), Args );
	}

	return Text;
}

