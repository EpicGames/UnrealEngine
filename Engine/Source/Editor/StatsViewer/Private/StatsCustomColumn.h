// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "IPropertyTableCustomColumn.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FProperty;
class FString;
class IPropertyHandle;
class IPropertyTableCell;
class IPropertyTableCellPresenter;
class IPropertyTableColumn;
class IPropertyTableUtilities;
class SWidget;
class UObject;

/**
 * A property table custom column used to display numerical data.
 * Also supports FVector2D UStruct properties
 * Will display totals in the column header if they are supplied in the 
 * TotalsMap
 */
class FStatsCustomColumn : public TSharedFromThis< FStatsCustomColumn >, public IPropertyTableCustomColumn
{
public:
	/** Begin IPropertyTableCustomColumn interface */
	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const override;
	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const override;
	/** End IPropertyTableCustomColumn interface */

	/** 
	 * Helper function to check if we can support displaying this property 
	 * @param	Property	The property to check
	 * @returns true if this property is supported for display by this column type
	 */
	static bool SupportsProperty( const FProperty* Property );

	/** 
	 * Helper function to get the text we would display for this property 
	 * @param	PropertyHandle	The property handle to retrieve the text from
	 * returns a string representation of the properties data, or an empty string if the property is unsupported
	 */
	static FText GetPropertyAsText( const TSharedPtr< IPropertyHandle > PropertyHandle, bool bGetRawValue = false);

public:

	/** The map we use to find our totals to display */
	TMap<FString, FText> TotalsMap;

private:

	/** 
	 * Delegate to display the total text 
	 * @param	Column	The column to get the total text for
	 */
	FText GetTotalText(TSharedRef< IPropertyTableColumn > Column) const;

private:

	/** The total object, if any */
	TWeakObjectPtr<UObject> TotalObject;
};

