// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookerStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookerStats)


UCookerStats::UCookerStats( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }
