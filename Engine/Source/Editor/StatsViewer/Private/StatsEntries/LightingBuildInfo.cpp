// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightingBuildInfo.h"
#include "UObject/Package.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Model.h"
#include "Lightmass/LightmappedSurfaceCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightingBuildInfo)

ULightingBuildInfo::ULightingBuildInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULightingBuildInfo::Set( TWeakObjectPtr<UObject> InObject, double InLightingTime, float InUnmappedTexelsPercentage, int32 InUnmappedTexelsMemory, int32 InTotalTexelMemory )
{
	Object = InObject;
	LightingTime = (float)InLightingTime;
	UnmappedTexelsPercentage = InUnmappedTexelsPercentage;
	UnmappedTexelsMemory = (float)InUnmappedTexelsMemory / 1024.0f;
	TotalTexelMemory = (float)InTotalTexelMemory / 1024.0f;

	UpdateNames();
}

void ULightingBuildInfo::UpdateNames()
{
	if( Object.IsValid() )
	{
		AActor* Actor = Cast<AActor>(Object.Get());
		ULightmappedSurfaceCollection* SurfaceCollection = Cast<ULightmappedSurfaceCollection>(Object.Get());

		if (SurfaceCollection && SurfaceCollection->SourceModel)
		{
			LevelName = SurfaceCollection->SourceModel->GetOutermost()->GetName();
		}
		else if (Actor)
		{
			LevelName = Actor->GetLevel()->GetOutermost()->GetName();
		}
		else
		{
			LevelName = Object->GetOutermost()->GetName();
		}

		const int32 NameIndex = LevelName.Find( TEXT("/"), ESearchCase::CaseSensitive);
		if ( NameIndex != INDEX_NONE )
		{
			LevelName.RightChopInline( NameIndex + 1, EAllowShrinking::No );
		}
	}
}
