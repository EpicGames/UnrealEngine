// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveStats.h"

#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrimitiveStats)

UPrimitiveStats::UPrimitiveStats(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPrimitiveStats::UpdateNames()
{
	if( Object.IsValid() )
	{
		Type = Object.Get()->GetClass()->GetName();
	}
}
