// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCookerStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShaderCookerStats)

UShaderCookerStats::UShaderCookerStats(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}
