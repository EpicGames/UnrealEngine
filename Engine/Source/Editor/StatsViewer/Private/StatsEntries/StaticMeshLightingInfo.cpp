// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshLightingInfo.h"
#include "UObject/Package.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Model.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshLightingInfo)

#define LOCTEXT_NAMESPACE "Editor.StatsViewer"

UStaticMeshLightingInfo::UStaticMeshLightingInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UStaticMeshLightingInfo::UpdateNames()
{
	if ( LevelName.Len() == 0 || StaticMeshActor.IsValid() )
	{
		const AActor* Actor = StaticMeshActor.Get();
		if (Actor)
		{
			LevelName = Actor->GetLevel()->GetOutermost()->GetName();
		}
		else
		{
			LevelName = TEXT("<None>");
		}

		const int32 NameIndex = LevelName.Find( TEXT("/"), ESearchCase::CaseSensitive);
		if ( NameIndex != INDEX_NONE )
		{
			LevelName.RightChopInline( NameIndex + 1, EAllowShrinking::No );
		}
	}

	if( bTextureMapping )
	{
		TextureMapping = LOCTEXT("LightingUsesTextureMapping", "Texture" ).ToString();
	}
	else
	{
		TextureMapping = LOCTEXT("LightingUsesVertexMapping", "Vertex" ).ToString();
	}
}

#undef LOCTEXT_NAMESPACE
