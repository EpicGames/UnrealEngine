// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureStats.h"

#include "UObject/WeakObjectPtr.h"

#include <cfloat>

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureStats)

UTextureStats::UTextureStats(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	LastTimeRendered( FLT_MAX )
{
}
