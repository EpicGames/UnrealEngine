// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPageManager.h"

#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IStatsPage.h"
#include "Misc/AssertionMacros.h"

const int32 FStatsPageManager::PageIdNone = -1;

FStatsPageManager& FStatsPageManager::Get()
{
	static FStatsPageManager* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FStatsPageManager;
	}
	return *Instance;
}

void FStatsPageManager::RegisterPage( int32 PageId, TSharedRef<IStatsPage> InPage )
{
	/** Ensure not already added */
	check(GetPageIndex(PageId) == -1);

	StatsPages.Add( InPage );
	StatsPageIds.Add( PageId );
}

void FStatsPageManager::RegisterPage( TSharedRef<class IStatsPage> InPage )
{
	StatsPages.Add(InPage);
	StatsPageIds.Add(PageIdNone);
}

void FStatsPageManager::UnregisterPage( TSharedRef<IStatsPage> InPage )
{
	int PageIndex = GetPageIndex( InPage );
	check(PageIndex >= 0);

	StatsPages.RemoveAt(PageIndex);
	StatsPageIds.Remove(PageIndex);
}

void FStatsPageManager::UnregisterAllPages()
{
	StatsPages.Empty();
	StatsPageIds.Empty();
}

int32 FStatsPageManager::NumPages() const
{
	return StatsPages.Num();
}

TSharedRef<class IStatsPage> FStatsPageManager::GetPageByIndex(int32 InPageIndex)
{
	check(InPageIndex < StatsPages.Num());
	return StatsPages[InPageIndex];
}

TSharedRef<IStatsPage> FStatsPageManager::GetPage( int32 InPageId )
{
	int PageIndex = GetPageIndex(InPageId);
	check(PageIndex >= 0);
	return StatsPages[PageIndex];
}

TSharedPtr<IStatsPage> FStatsPageManager::GetPage( const FName& InPageName )
{
	for( auto Iter = StatsPages.CreateIterator(); Iter; Iter++ )
	{
		TSharedRef<class IStatsPage> Page  = *Iter;
		if(Page.Get().GetName() == InPageName)
		{
			return Page;
		}
	}

	return NULL;
}

int FStatsPageManager::GetPageIndex(TSharedRef<IStatsPage> InPage) const
{
	for (int PageIndex = 0; PageIndex < StatsPages.Num(); ++PageIndex)
	{
		if (InPage == StatsPages[PageIndex])
		{
			return PageIndex;
		}
	}
	
	return -1;
}

int FStatsPageManager::GetPageIndex(int32 PageId) const
{
	for (int PageIndex = 0; PageIndex < StatsPages.Num(); ++PageIndex)
	{
		if (PageId == StatsPageIds[PageIndex])
		{
			return PageIndex;
		}
	}
	
	return -1;
}
