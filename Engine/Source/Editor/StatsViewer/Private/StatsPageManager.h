// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class IStatsPage;

/**
 * A class which manages a the collection of known stats pages           
 */
class FStatsPageManager
{
public:
	FStatsPageManager(const FName& InName = NAME_None) : Name( InName )	{}
	virtual ~FStatsPageManager() = default;

	/** Gets the instance of this manager */
	static FStatsPageManager& Get();

	const FName& GetName() const { return Name; }

	/** 
	 * Register a page with the manager 
	 * @param	PageId	Use specific page id (must be unique)
	 * @param	InPage	The page to register
	 */
	void RegisterPage( int32 PageId, TSharedRef<class IStatsPage> InPage );

	/**
	 * Register a page with the manager
	 * @param	InPage	The page to register
	 */
	void RegisterPage( TSharedRef<class IStatsPage> InPage );

	/** 
	 * Unregister a page from the manager 
	 * @param	InPage	The page to unregister
	 */
	void UnregisterPage( TSharedRef<class IStatsPage> InPage );

	/** Unregister & delete all registered pages */
	void UnregisterAllPages();

	/** Get the number of stats pages we have */
	int32 NumPages() const;

	/** Get the page at the specified index */
	TSharedRef<class IStatsPage> GetPageByIndex( int32 InPageIndex );

	/** Get page by its type */
	TSharedRef<class IStatsPage> GetPage( int32 InPageId );

	/** Get the factory with the specified name */
	TSharedPtr<class IStatsPage> GetPage( const FName& InPageName );

private:
	int GetPageIndex(int32 PageId) const;
	int GetPageIndex(TSharedRef< IStatsPage > InPage) const;

private:
	static const int32 PageIdNone; // = -1;

	/** The registered pages */
	TArray< TSharedRef<class IStatsPage> > StatsPages;
	TArray< int32 > StatsPageIds;

	FName Name;
};
