// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/CookerStatsPage.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "AssetRegistry/AssetRegistryModule.h"


#define LOCTEXT_NAMESPACE "FCookerStatsPage"


/* FCookerStatsPage static functions
 *****************************************************************************/

FCookerStatsPage& FCookerStatsPage::Get()
{
	static FCookerStatsPage* Instance = nullptr;

	if (Instance == nullptr)
	{
		Instance = new FCookerStatsPage;
	}

	return *Instance;
}


/* IStatsPage interface
 *****************************************************************************/

void FCookerStatsPage::Generate( TArray<TWeakObjectPtr<UObject>>& OutObjects ) const
{
	// visitor for the directory enumerator
	class FPlatformDirectoryVisitor
		: public IPlatformFile::FDirectoryVisitor
	{
	public:

		FPlatformDirectoryVisitor( IAssetRegistry& InAssetRegistry, TArray<TWeakObjectPtr<UObject>>& InOutObjects, const FString& InPlatformName )
			: AssetRegistry(InAssetRegistry)
			, OutObjects(InOutObjects)
			, PlatformName(InPlatformName)
		{ }

		virtual bool Visit( const TCHAR* FilenameOrDirectory, bool bIsDirectory ) override
		{
			if (!bIsDirectory)
			{
				FString OriginalFileName = FString(FilenameOrDirectory).Replace(*(FPaths::ProjectSavedDir() / TEXT("Cooked") / PlatformName), *FPaths::RootDir());

				if (OriginalFileName.EndsWith(TEXT(".uasset")) || OriginalFileName.EndsWith(TEXT(".umap")))
				{
					UCookerStats* Entry = NewObject<UCookerStats>();
					Entry->AddToRoot();
					Entry->Path = FilenameOrDirectory;
					Entry->SizeAfter = (int32)IFileManager::Get().FileSize(FilenameOrDirectory) / 1024.0f;
					Entry->SizeBefore = FMath::Max(0.0f, (int32)IFileManager::Get().FileSize(*OriginalFileName) / 1024.0f);

					FString PackageName;

					if (FPackageName::TryConvertFilenameToLongPackageName(OriginalFileName, PackageName))
					{
						TArray<FAssetData> Assets;
						AssetRegistry.GetAssetsByPackageName(FName(*PackageName), Assets);

						for (auto Asset : Assets)
						{
							Entry->Assets.Add(Asset.GetAsset());
						}
					}

					OutObjects.Add(Entry);
				}
			}

			return true;
		}

	private:

		IAssetRegistry& AssetRegistry;
		TArray<TWeakObjectPtr<UObject>>& OutObjects;
		FString PlatformName;
	};


	if (SelectedPlatformName.IsEmpty())
	{
		return;
	}

	// create data objects for each cooked asset
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FPlatformDirectoryVisitor PlatformDirectoryVisitor(AssetRegistryModule.Get(), OutObjects, SelectedPlatformName);
	FPlatformFileManager::Get().GetPlatformFile().IterateDirectoryRecursively(*(FPaths::ProjectSavedDir() / TEXT("Cooked") / SelectedPlatformName), PlatformDirectoryVisitor);
}


void FCookerStatsPage::GenerateTotals( const TArray<TWeakObjectPtr<UObject>>& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if (InObjects.Num() == 0)
	{
		return;
	}

	UCookerStats* TotalEntry = NewObject<UCookerStats>();
	{
		for (auto It = InObjects.CreateConstIterator(); It; ++It)
		{
			UCookerStats* StatsEntry = Cast<UCookerStats>(It->Get());
			
			TotalEntry->SizeBefore += StatsEntry->SizeBefore;
			TotalEntry->SizeAfter += StatsEntry->SizeAfter;
		}

		OutTotals.Add(TEXT("SizeBefore"), FText::AsNumber(TotalEntry->SizeBefore));
		OutTotals.Add(TEXT("SizeAfter"), FText::AsNumber(TotalEntry->SizeAfter));
	}
}


TSharedPtr<SWidget> FCookerStatsPage::GetCustomFilter( TWeakPtr<IStatsViewer> InParentStatsViewer )
{
	return SNew(SComboButton)
		.ContentPadding(2.0f)
		.Visibility_Raw(this, &FCookerStatsPage::HandleFilterComboButtonVisibility)
		.OnGetMenuContent_Raw(this, &FCookerStatsPage::HandleFilterComboButtonGetMenuContent)
		.ButtonContent()
		[
			SNew(STextBlock)
				.Text_Raw(this, &FCookerStatsPage::HandleFilterComboButtonText)
				.ToolTipText(LOCTEXT("FilterColumnToUse_Tooltip", "Choose the target platform to filter when searching"))
		];
}


void FCookerStatsPage::OnShow( TWeakPtr<class IStatsViewer> InParentStatsViewer )
{

}


void FCookerStatsPage::OnHide( )
{

}


/* FCookerStatsPage callback
 *****************************************************************************/

TSharedRef<SWidget> FCookerStatsPage::HandleFilterComboButtonGetMenuContent( ) const
{
	// visitor for the directory enumerator
	class FCookedDirectoryVisitor
		: public IPlatformFile::FDirectoryVisitor
	{
	public:

		FCookedDirectoryVisitor( const FCookerStatsPage& InCookerStatsPage, FMenuBuilder& InMenuBuilder )
			: CookerStatsPage(InCookerStatsPage)
			, MenuBuilder(InMenuBuilder)
		{ }

		virtual bool Visit( const TCHAR* FilenameOrDirectory, bool bIsDirectory ) override
		{
			if (bIsDirectory)
			{
				FString PlatformName = FPaths::GetBaseFilename(FilenameOrDirectory);

				MenuBuilder.AddMenuEntry(
					FText::FromString(PlatformName),
					FText::FromString(PlatformName),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(const_cast<FCookerStatsPage*>(&CookerStatsPage), &FCookerStatsPage::HandleFilterMenuEntryExecute, PlatformName),
						FCanExecuteAction(),
						FIsActionChecked::CreateRaw(&CookerStatsPage, &FCookerStatsPage::HandleFilterMenuEntryIsChecked, PlatformName)
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}

			return true;
		}

	private:

		const FCookerStatsPage& CookerStatsPage;
		FMenuBuilder& MenuBuilder;
	};


	// create menu entries for each cooked directory
	FMenuBuilder MenuBuilder(true, nullptr);
	{
		FCookedDirectoryVisitor CookedDirectoryVisitor(*this, MenuBuilder);
		FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(*(FPaths::ProjectSavedDir() / TEXT("Cooked")), CookedDirectoryVisitor);
	}

	return MenuBuilder.MakeWidget();
}


FText FCookerStatsPage::HandleFilterComboButtonText( ) const
{
	if (SelectedPlatformName.IsEmpty())
	{
		return LOCTEXT("SelectPlatformLabel", "Select platform...");
	}

	return FText::FromString(SelectedPlatformName);
}


EVisibility FCookerStatsPage::HandleFilterComboButtonVisibility( ) const
{
	return EVisibility::Visible;
}


void FCookerStatsPage::HandleFilterMenuEntryExecute( FString PlatformName )
{
	SelectedPlatformName = PlatformName;

	Refresh();
}


bool FCookerStatsPage::HandleFilterMenuEntryIsChecked( FString PlatformName ) const
{
	return (PlatformName == SelectedPlatformName);
}


#undef LOCTEXT_NAMESPACE
