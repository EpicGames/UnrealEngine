// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "IStatsViewer.h"
#include "StatsPage.h"
#include "CookerStats.h"

/**
 * Implements a Stats page representing cooker statistics.
 */
class FCookerStatsPage
	: public FStatsPage<UCookerStats>
{
public:

	/** Singleton accessor. */
	static FCookerStatsPage& Get( );

	/** Virtual destructor. */
	virtual ~FCookerStatsPage( ) = default;

public:

	// IStatsPage interface

	virtual void Generate( TArray<TWeakObjectPtr<UObject>>& OutObjects ) const override;
	virtual void GenerateTotals( const TArray<TWeakObjectPtr<UObject>>& InObjects, TMap<FString, FText>& OutTotals ) const override;
	virtual TSharedPtr<SWidget> GetCustomFilter( TWeakPtr<class IStatsViewer> InParentStatsViewer ) override;
	virtual void OnShow( TWeakPtr<class IStatsViewer> InParentStatsViewer ) override;
	virtual void OnHide( ) override;

private:

	/** Callback for getting the filter combo button's menu content. */
	TSharedRef<SWidget> HandleFilterComboButtonGetMenuContent( ) const;

	/** Callback for getting the filter combo button's text label. */
	FText HandleFilterComboButtonText( ) const;

	/** Callback for getting the filter combo button's visibility. */
	EVisibility HandleFilterComboButtonVisibility( ) const;

	/** Callback for executing a filter menu entry. */
	void HandleFilterMenuEntryExecute( FString PlatformName );

	/** Callback for determining whether a filter menu entry should be marked checked. */
	bool HandleFilterMenuEntryIsChecked( FString PlatformName ) const;

private:

	FString SelectedPlatformName;
};
