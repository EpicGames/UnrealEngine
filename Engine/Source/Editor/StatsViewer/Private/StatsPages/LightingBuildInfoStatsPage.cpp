// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/LightingBuildInfoStatsPage.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer.LightingBuildInfo"

FLightingBuildInfoStatsPage& FLightingBuildInfoStatsPage::Get()
{
	static FLightingBuildInfoStatsPage* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FLightingBuildInfoStatsPage;
	}
	return *Instance;
}

void FLightingBuildInfoStatsPage::Clear()
{
	for( auto It = Entries.CreateIterator(); It; ++It )
	{
		TWeakObjectPtr<ULightingBuildInfo> Entry = *It;
		if(Entry.IsValid())
		{
			Entry->RemoveFromRoot();
		}
	}

	Entries.Empty();
}

void FLightingBuildInfoStatsPage::AddEntry( UObject* InEntry )
{
	if( InEntry->IsA( ULightingBuildInfo::StaticClass() ) )
	{
		ULightingBuildInfo* LightingBuildInfo = Cast<ULightingBuildInfo>(InEntry);
		LightingBuildInfo->AddToRoot();
		Entries.Add( LightingBuildInfo );
	}		
}

void FLightingBuildInfoStatsPage::Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const
{
	if(Entries.Num())
	{		
		for( auto It = Entries.CreateConstIterator(); It; ++It )
		{
			TWeakObjectPtr<ULightingBuildInfo> Entry = *It;
			if(Entry.IsValid())
			{
				ULightingBuildInfo* NewObject = DuplicateObject<ULightingBuildInfo>(Entry.Get(), Entry->GetOuter());
				NewObject->AddToRoot();
				OutObjects.Add( NewObject );
			}
		}
	}
}

void FLightingBuildInfoStatsPage::GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if(InObjects.Num())
	{
		ULightingBuildInfo* TotalEntry = NewObject<ULightingBuildInfo>();
		for( auto It = InObjects.CreateConstIterator(); It; ++It )
		{
			ULightingBuildInfo* Entry = Cast<ULightingBuildInfo>( It->Get() );
			TotalEntry->LightingTime += Entry->LightingTime;
			TotalEntry->UnmappedTexelsPercentage += Entry->UnmappedTexelsPercentage;
			TotalEntry->UnmappedTexelsMemory += Entry->UnmappedTexelsMemory;
			TotalEntry->TotalTexelMemory += Entry->TotalTexelMemory;
		}

		OutTotals.Add( TEXT("LightingTime"), FText::AsNumber( TotalEntry->LightingTime ) );
		OutTotals.Add( TEXT("UnmappedTexelsPercentage"), FText::AsNumber( TotalEntry->UnmappedTexelsPercentage ) );
		OutTotals.Add( TEXT("UnmappedTexelsMemory"), FText::AsNumber( TotalEntry->UnmappedTexelsMemory ) );
		OutTotals.Add( TEXT("TotalTexelMemory"), FText::AsNumber( TotalEntry->TotalTexelMemory ) );
	}
}

#undef LOCTEXT_NAMESPACE
