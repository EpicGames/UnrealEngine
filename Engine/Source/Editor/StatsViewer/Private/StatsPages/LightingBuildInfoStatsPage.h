// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StatsPage.h"
#include "LightingBuildInfo.h"

/** Stats page representing lighting build info */
class FLightingBuildInfoStatsPage : public FStatsPage<ULightingBuildInfo>
{
public:
	/** Singleton accessor */
	static FLightingBuildInfoStatsPage& Get();

	/** Begin IStatsPage interface */
	virtual void Clear() override;
	virtual void AddEntry( UObject* InEntry ) override;
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const override;
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const override;
	/** End IStatsPage interface */

	virtual ~FLightingBuildInfoStatsPage() = default;

private:

	/** 
	 * The list of objects we will display. This stat entry differs from others in that the
	 * data it displays is not derived directly from the scene/assets, rather it is collected at
	 * build time and stored here
	 */
	TArray< TWeakObjectPtr<ULightingBuildInfo> > Entries;
	
};

