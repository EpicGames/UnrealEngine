// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/PrimitiveStatsPage.h"
#include "Engine/Level.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCountMem.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Model.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ModelComponent.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "UObject/UObjectIterator.h"
#include "SceneInterface.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "LandscapeProxy.h"
#include "LightMap.h"
#include "LandscapeComponent.h"
#include "Engine/LevelStreaming.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Components/InstancedStaticMeshComponent.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer.PrimitiveStats"

FPrimitiveStatsPage& FPrimitiveStatsPage::Get()
{
	static FPrimitiveStatsPage* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FPrimitiveStatsPage;
	}
	return *Instance;
}

/** Helper class to generate statistics */
struct PrimitiveStatsGenerator
{
	/** Map a list of the resources with the data about them */
	TMap<UObject*, UPrimitiveStats*> ResourceToStatsMap;

	/** Check if the object is in a visible level */
	bool IsInVisibleLevel( UObject* InObject, UWorld* InWorld ) const
	{
		check( InObject );
		check( InWorld );

		UObject* ObjectTopMostPackage = InObject->GetOutermostObject()->GetPackage();

		for( int32 LevelIndex=0; LevelIndex<InWorld->GetNumLevels(); LevelIndex++ )
		{
			ULevel* Level = InWorld->GetLevel(LevelIndex);
			if( Level && Level->GetPackage() == ObjectTopMostPackage )
			{
				return true;
			}
		}
		return false;
	}

	/** Add a new statistic to the internal map (or update an existing one) from the supplied component */
	UPrimitiveStats* Add(UPrimitiveComponent* InPrimitiveComponent, EPrimitiveObjectSets InObjectSet, bool bAllowTransientWorld)
	{
		// Transient objects are not part of level, but if part of transient package, we can allow depending on flag bAllowTransientWorld.
		if( (!bAllowTransientWorld && InPrimitiveComponent->GetOutermost() == GetTransientPackage()) || InPrimitiveComponent->HasAnyFlags( RF_Transient ) )
		{
			return NULL;
		}

		// Owned by a default object? Not part of a level either.
		if(InPrimitiveComponent->GetOuter() && InPrimitiveComponent->GetOuter()->IsTemplate())
		{
			return NULL;
		}

		UStaticMeshComponent*	StaticMeshComponent		= Cast<UStaticMeshComponent>(InPrimitiveComponent);
		UInstancedStaticMeshComponent*	InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(InPrimitiveComponent);
		UModelComponent*		ModelComponent			= Cast<UModelComponent>(InPrimitiveComponent);
		USkeletalMeshComponent*	SkeletalMeshComponent	= Cast<USkeletalMeshComponent>(InPrimitiveComponent);
		ULandscapeComponent*	LandscapeComponent		= Cast<ULandscapeComponent>(InPrimitiveComponent);
		UObject*				Resource				= NULL;
		AActor*					ActorOuter				= Cast<AActor>(InPrimitiveComponent->GetOuter());

		int32 VertexColorMem		= 0;
		int32 InstVertexColorMem	= 0;
		// Calculate number of direct and other lights relevant to this component.
		int32 LightsLMCount			= 0;
		int32 LightsOtherCount		= 0;
		bool bUsesOnlyUnlitMaterials = InPrimitiveComponent->UsesOnlyUnlitMaterials();
		int32 NumHWInstances = InstancedStaticMeshComponent ? InstancedStaticMeshComponent->GetNumRenderInstances() : 1;

		// The static mesh is a static mesh component's resource.
		if( StaticMeshComponent )
		{
			UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();
			Resource = Mesh;

			// Calculate vertex color memory on the actual mesh.
			if( Mesh && Mesh->GetRenderData())
			{
				// Accumulate memory for each LOD
				for( int32 LODIndex = 0; LODIndex < Mesh->GetRenderData()->LODResources.Num(); ++LODIndex )
				{
					VertexColorMem += Mesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.ColorVertexBuffer.GetAllocatedSize();
				}
			}

			// Calculate instanced vertex color memory used on the component.
			for( int32 LODIndex = 0; LODIndex < StaticMeshComponent->LODData.Num(); ++LODIndex )
			{
				// Accumulate memory for each LOD
				const FStaticMeshComponentLODInfo& LODInfo = StaticMeshComponent->LODData[ LODIndex ];
				if( LODInfo.OverrideVertexColors )
				{
					InstVertexColorMem += LODInfo.OverrideVertexColors->GetAllocatedSize();	
				}
			}
			// Calculate the number of lightmap and shadow map lights
			if( !bUsesOnlyUnlitMaterials )
			{
				if( StaticMeshComponent->LODData.Num() > 0 )
				{
					FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[0];
					const FMeshMapBuildData* MeshMapBuildData = StaticMeshComponent->GetMeshMapBuildData(ComponentLODInfo);
					if( MeshMapBuildData && MeshMapBuildData->LightMap )
					{
						LightsLMCount = MeshMapBuildData->LightMap->LightGuids.Num();
					}
				}
			}
		}
		// A model component is its own resource.
		else if( ModelComponent )			
		{
			// Make sure model component is referenced by level.
			ULevel* Level = CastChecked<ULevel>(ModelComponent->GetOuter());
			if( Level->ModelComponents.Find( ModelComponent ) != INDEX_NONE )
			{
				Resource = ModelComponent->GetModel();

				// Calculate the number of lightmap and shadow map lights
				if( !bUsesOnlyUnlitMaterials )
				{
					const TIndirectArray<FModelElement> Elements = ModelComponent->GetElements();
					if( Elements.Num() > 0 )
					{
						const FMeshMapBuildData* MeshMapBuildData = Elements[0].GetMeshMapBuildData();
						if( MeshMapBuildData && MeshMapBuildData->LightMap )
						{
							LightsLMCount = MeshMapBuildData->LightMap->LightGuids.Num();
						}
					}
				}
			}
		}
		// The skeletal mesh of a skeletal mesh component is its resource.
		else if( SkeletalMeshComponent )
		{
			USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
			Resource = Mesh;
			// Calculate vertex color usage for skeletal meshes
			if( Mesh )
			{
				FSkeletalMeshRenderData* SkelMeshRenderData = Mesh->GetResourceForRendering();
				for( int32 LODIndex = 0; LODIndex < SkelMeshRenderData->LODRenderData.Num(); ++LODIndex )
				{
					const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[ LODIndex ];
					VertexColorMem += LODData.StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize();
				}
			}
		}
		// The landscape of a landscape component is its resource.
		else if (LandscapeComponent)
		{
			Resource = LandscapeComponent->GetLandscapeProxy();
			const FMeshMapBuildData* MeshMapBuildData = LandscapeComponent->GetMeshMapBuildData();

			if (MeshMapBuildData && MeshMapBuildData->LightMap)
			{
				LightsLMCount = MeshMapBuildData->LightMap->LightGuids.Num();
			}
		}

		UWorld* World = InPrimitiveComponent->GetWorld();
	//	check(World); // @todo: re-instate this check once the GWorld migration has completed
		/// If we should skip the actor. Skip if the actor has no outer or if we are only showing selected actors and the actor isn't selected
		const bool bShouldSkip = ActorOuter == NULL || (ActorOuter != NULL && InObjectSet == PrimitiveObjectSets_SelectedObjects && ActorOuter->IsSelected() == false );
		// Dont' care about components without a resource.
		if(	Resource 
			&& World != NULL
			// Require actor association for selection and to disregard mesh emitter components. The exception being model components.
			&&	(!bShouldSkip || (ModelComponent && InObjectSet != PrimitiveObjectSets_SelectedObjects ) )
			// Only list primitives in visible levels
			&&	IsInVisibleLevel( InPrimitiveComponent, World ) 
			// Don't list pending kill components.
			&&	IsValid(InPrimitiveComponent) )
		{
			// Retrieve number of relevant lights.
			const int32 RelevantLightCount = World->Scene->GetRelevantLightCount(InPrimitiveComponent);

			// Only look for relevant lights if we aren't unlit.
			if( !bUsesOnlyUnlitMaterials )
			{
				// Lightmap and shadow map lights are calculated above, per component type, infer the "other" light count here
				LightsOtherCount = RelevantLightCount >= LightsLMCount ? RelevantLightCount - LightsLMCount : 0;
			}

			// Figure out memory used by light and shadow maps and light/ shadow map resolution.
			int32 LightMapWidth			= 0;
			int32 LightMapHeight		= 0;
			InPrimitiveComponent->GetLightMapResolution( LightMapWidth, LightMapHeight );
			int32 LMSMResolution		= FMath::TruncToInt32(FMath::Sqrt( static_cast<float>(LightMapHeight * LightMapWidth) ));
			int32 LightMapData			= 0;
			int32 LegacyShadowMapData	= 0;
			InPrimitiveComponent->GetLightAndShadowMapMemoryUsage( LightMapData, LegacyShadowMapData );

			// Check whether we already have an entry for the associated static mesh.
			UPrimitiveStats** StatsEntryPtr = ResourceToStatsMap.Find( Resource );
			if( StatsEntryPtr )
			{
				check(*StatsEntryPtr);
				UPrimitiveStats* StatsEntry = *StatsEntryPtr;

				// We do. Update existing entry.
				StatsEntry->Count++;
				StatsEntry->HWInstances		+= NumHWInstances;
				StatsEntry->Actors.AddUnique(ActorOuter);
				StatsEntry->RadiusMin		= FMath::Min( StatsEntry->RadiusMin, InPrimitiveComponent->Bounds.SphereRadius );
				StatsEntry->RadiusMax		= FMath::Max( StatsEntry->RadiusMax, InPrimitiveComponent->Bounds.SphereRadius );
				StatsEntry->RadiusAvg		+= InPrimitiveComponent->Bounds.SphereRadius;
				StatsEntry->LightsLM		+= LightsLMCount;
				StatsEntry->LightsOther		+= LightsOtherCount;
				StatsEntry->LightMapData	+= (float)LightMapData / 1024.0f;
				StatsEntry->LMSMResolution	+= LMSMResolution;
				StatsEntry->UpdateNames();

				if ( !ModelComponent && !LandscapeComponent )
				{
					// Count instanced sections
					StatsEntry->InstSections += StatsEntry->Sections * NumHWInstances;
					StatsEntry->InstTriangles += StatsEntry->Triangles * NumHWInstances;
				}

				// ... in the case of a model component (aka BSP).
				if( ModelComponent )
				{
					// If Count represents the Model itself, we do NOT want to increment it now.
					StatsEntry->Count--;

					for (const auto& Element : ModelComponent->GetElements())
					{
						StatsEntry->Triangles += Element.NumTriangles;
						StatsEntry->Sections++;
					}

					StatsEntry->InstSections = StatsEntry->Sections;
					StatsEntry->InstTriangles = StatsEntry->Triangles;
				}
				else if( StaticMeshComponent )
				{
					// This stat is used by multiple components so accumulate instanced vertex color memory.
					StatsEntry->InstVertexColorMem += (float)InstVertexColorMem / 1024.0f;
				}
				else if (LandscapeComponent)
				{
					// If Count represents the Landscape itself, we do NOT want to increment it now.
					StatsEntry->Count--;
				}
			}
			else
			{
				// We don't. Create new base entry.
				UPrimitiveStats* NewStatsEntry = NewObject<UPrimitiveStats>();
				NewStatsEntry->AddToRoot();
				NewStatsEntry->Object			= Resource;
				NewStatsEntry->Actors.AddUnique(ActorOuter);
				NewStatsEntry->Count			= 1;
				NewStatsEntry->HWInstances		= NumHWInstances;
				NewStatsEntry->Triangles		= 0;
				NewStatsEntry->InstTriangles	= 0;
				NewStatsEntry->ResourceSize		= (float)(FArchiveCountMem(Resource).GetNum() + Resource->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal)) / 1024.0f;
				NewStatsEntry->Sections			= 0;
				NewStatsEntry->InstSections		= 0;
				NewStatsEntry->RadiusMin		= InPrimitiveComponent->Bounds.SphereRadius;
				NewStatsEntry->RadiusAvg		= InPrimitiveComponent->Bounds.SphereRadius;
				NewStatsEntry->RadiusMax		= InPrimitiveComponent->Bounds.SphereRadius;
				NewStatsEntry->LightsLM			= LightsLMCount;
				NewStatsEntry->LightsOther		= (float)LightsOtherCount;
				NewStatsEntry->LightMapData		= (float)LightMapData / 1024.0f;
				NewStatsEntry->LMSMResolution	= (float)LMSMResolution;
				NewStatsEntry->VertexColorMem	= (float)VertexColorMem / 1024.0f;
				NewStatsEntry->InstVertexColorMem = (float)InstVertexColorMem / 1024.0f;
				NewStatsEntry->UpdateNames();

				// Fix up triangle and section count...

				// ... in the case of a static mesh component.
				if( StaticMeshComponent )
				{
					UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
					if( StaticMesh && StaticMesh->GetRenderData())
					{
						for( int32 SectionIndex=0; SectionIndex<StaticMesh->GetRenderData()->LODResources[0].Sections.Num(); SectionIndex++ )
						{
							const FStaticMeshSection& StaticMeshSection = StaticMesh->GetRenderData()->LODResources[0].Sections[SectionIndex];
							NewStatsEntry->Triangles	+= StaticMeshSection.NumTriangles;
							NewStatsEntry->Sections++;
						}
					}
				}
				// ... in the case of a model component (aka BSP).
				else if( ModelComponent )
				{
					TIndirectArray<FModelElement> Elements = ModelComponent->GetElements();
					for( int32 ElementIndex=0; ElementIndex<Elements.Num(); ElementIndex++ )
					{
						const FModelElement& Element = Elements[ElementIndex];
						NewStatsEntry->Triangles += Element.NumTriangles;
						NewStatsEntry->Sections++;
					}

				}
				// ... in the case of skeletal mesh component.
				else if( SkeletalMeshComponent )
				{
					USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
					if( SkeletalMesh )
					{
						FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering();
						if (SkelMeshRenderData->LODRenderData.Num())
						{
							const FSkeletalMeshLODRenderData& BaseLOD = SkelMeshRenderData->LODRenderData[0];
							for( int32 SectionIndex=0; SectionIndex<BaseLOD.RenderSections.Num(); SectionIndex++ )
							{
								const FSkelMeshRenderSection& Section = BaseLOD.RenderSections[SectionIndex];
								NewStatsEntry->Triangles += Section.NumTriangles;
								NewStatsEntry->Sections++;
							}
						}
					}
				}
				else if (LandscapeComponent)
				{
					TSet<UTexture2D*> UniqueTextures;
					for (auto ItComponents = LandscapeComponent->GetLandscapeProxy()->LandscapeComponents.CreateConstIterator(); ItComponents; ++ItComponents)
					{
						const ULandscapeComponent* CurrentComponent = *ItComponents;

						// count triangles and sections in the landscape
						NewStatsEntry->Triangles += FMath::Square(CurrentComponent->ComponentSizeQuads) * 2;
						NewStatsEntry->Sections += FMath::Square(CurrentComponent->NumSubsections);

						// count resource usage of landscape
						bool bNotUnique = false;
						UniqueTextures.Add(CurrentComponent->GetHeightmap(), &bNotUnique);
						if (!bNotUnique)
						{
							const SIZE_T HeightmapResourceSize = CurrentComponent->GetHeightmap()->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
							NewStatsEntry->ResourceSize += (float)HeightmapResourceSize / 1024.0f;
						}

						const TArray<UTexture2D*>& ComponentWeightmapTextures = CurrentComponent->GetWeightmapTextures();

						for (UTexture2D* Weightmap : ComponentWeightmapTextures)
						{
							UniqueTextures.Add(Weightmap, &bNotUnique);
							if (!bNotUnique)
							{
								const SIZE_T WeightmapResourceSize = Weightmap->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
								NewStatsEntry->ResourceSize += (float)WeightmapResourceSize / 1024.f;
							}
						}
					}
				}

				NewStatsEntry->InstTriangles = NewStatsEntry->Triangles * NumHWInstances;
				NewStatsEntry->InstSections = NewStatsEntry->Sections * NumHWInstances;

				// Add to map.
				ResourceToStatsMap.Add( Resource, NewStatsEntry );

				return NewStatsEntry;
			}
		}

		return NULL;
	}

	/** Called once all stats are gathered into the map */
	void Generate()
	{
		// consolidate averages etc.
		for( auto It = ResourceToStatsMap.CreateIterator(); It; ++It )
		{
			It.Value()->InstTriangles	= It.Value()->Count * It.Value()->Triangles;
			It.Value()->LightsTotal		= ( (float)It.Value()->LightsLM + It.Value()->LightsOther ) / (float)It.Value()->Count;
			It.Value()->ObjLightCost	= It.Value()->LightsOther * It.Value()->Sections;
			It.Value()->LightsOther		= It.Value()->LightsOther / It.Value()->Count;
			It.Value()->RadiusAvg		/= It.Value()->Count;
			It.Value()->LMSMResolution	/= It.Value()->Count;
		}
	}
};

void FPrimitiveStatsPage::Generate(TArray< TWeakObjectPtr<UObject> >& OutObjects) const
{
	PrimitiveStatsGenerator Generator;
	Generator.Generate();

	UWorld* World = GetWorld();
	bool bAllowTransientWorld = false;

	if (StatsWorld.IsValid())
	{
		bAllowTransientWorld = (StatsWorld->GetOutermost() == GetTransientPackage());
	}

	switch ((EPrimitiveObjectSets)ObjectSetIndex)
	{
		case PrimitiveObjectSets_CurrentLevel:
		{
			for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
			{
				AActor* Owner = (*It)->GetOwner();

				if (Owner != nullptr && !Owner->HasAnyFlags(RF_ClassDefaultObject) && Owner->IsInLevel(World->GetCurrentLevel()))
				{
					UPrimitiveStats* StatsEntry = Generator.Add(*It, (EPrimitiveObjectSets)ObjectSetIndex, bAllowTransientWorld);

					if (StatsEntry != nullptr)
					{
						OutObjects.Add(StatsEntry);
					}
				}
			}
		}
		break;

		case PrimitiveObjectSets_AllObjects:
		{
			if (World)
			{
				TArray<ULevel*> Levels;

				// Add main level.
				Levels.AddUnique(World->PersistentLevel);

				// Add secondary levels.
				for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
				{
					if (StreamingLevel)
					{
						if (ULevel* Level = StreamingLevel->GetLoadedLevel())
						{
							Levels.AddUnique(Level);
						}
					}
				}

				for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
				{
					AActor* Owner = (*It)->GetOwner();

					if (Owner != nullptr && !Owner->HasAnyFlags(RF_ClassDefaultObject))
					{
						ULevel* CheckLevel = Owner->GetLevel();

						if (CheckLevel != nullptr && (Levels.Contains(CheckLevel)))
						{
							UPrimitiveStats* StatsEntry = Generator.Add(*It, (EPrimitiveObjectSets)ObjectSetIndex, bAllowTransientWorld);

							if (StatsEntry != nullptr)
							{
								OutObjects.Add(StatsEntry);
							}
						}
					}
				}
			}
		}
		break;

		case PrimitiveObjectSets_SelectedObjects:
		{
			TArray<UObject*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), SelectedActors);

			for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
			{
				AActor* Owner = (*It)->GetOwner();
				if (Owner != nullptr && !Owner->HasAnyFlags(RF_ClassDefaultObject) && SelectedActors.Contains(Owner))
				{
					UPrimitiveStats* StatsEntry = Generator.Add(*It, (EPrimitiveObjectSets)ObjectSetIndex, bAllowTransientWorld);
					if (StatsEntry != nullptr)
					{
						OutObjects.Add(StatsEntry);
					}
				}
			}
		}
		break;
	}
}

void FPrimitiveStatsPage::GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if(InObjects.Num())
	{
		UPrimitiveStats* TotalEntry = NewObject<UPrimitiveStats>();

		TotalEntry->RadiusMin = TNumericLimits<double>::Max();
		TotalEntry->RadiusMax = 0.0f;

		// build total entry
		for( auto It = InObjects.CreateConstIterator(); It; ++It )
		{
			UPrimitiveStats* StatsEntry = Cast<UPrimitiveStats>( It->Get() );

			TotalEntry->Count += StatsEntry->Count;
			TotalEntry->Sections += StatsEntry->Sections;
			TotalEntry->HWInstances += StatsEntry->HWInstances;
			TotalEntry->InstSections += StatsEntry->InstSections;
			TotalEntry->Triangles += StatsEntry->Triangles;
			TotalEntry->InstTriangles += StatsEntry->InstTriangles;
			TotalEntry->ResourceSize += StatsEntry->ResourceSize;
			TotalEntry->VertexColorMem += StatsEntry->VertexColorMem;
			TotalEntry->InstVertexColorMem += StatsEntry->InstVertexColorMem;
			TotalEntry->LightsLM += StatsEntry->LightsLM;
			TotalEntry->LightsOther += StatsEntry->LightsOther;
			TotalEntry->LightsTotal += StatsEntry->LightsTotal;
			TotalEntry->ObjLightCost += FMath::TruncToFloat( StatsEntry->ObjLightCost );
			TotalEntry->LightMapData += StatsEntry->LightMapData;
			TotalEntry->LMSMResolution += StatsEntry->LMSMResolution;
			TotalEntry->RadiusMin = FMath::Min(TotalEntry->RadiusMin, StatsEntry->RadiusMin);
			TotalEntry->RadiusMax = FMath::Max(TotalEntry->RadiusMax, StatsEntry->RadiusMax);
			TotalEntry->RadiusAvg += StatsEntry->RadiusAvg;
		}

		TotalEntry->LMSMResolution /= InObjects.Num();
		TotalEntry->LightsTotal /= InObjects.Num();
		TotalEntry->LightsOther /= InObjects.Num();
		TotalEntry->RadiusAvg /= InObjects.Num();

		OutTotals.Add( TEXT("Count"), FText::AsNumber( TotalEntry->Count ) );
		OutTotals.Add( TEXT("HWInstances"), FText::AsNumber(TotalEntry->HWInstances) );
		OutTotals.Add( TEXT("InstSections"), FText::AsNumber( TotalEntry->InstSections ) );
		OutTotals.Add( TEXT("Triangles"), FText::AsNumber( TotalEntry->Triangles ) );
		OutTotals.Add( TEXT("InstTriangles"), FText::AsNumber( TotalEntry->InstTriangles ) );
		OutTotals.Add( TEXT("ResourceSize"), FText::AsNumber( TotalEntry->ResourceSize ) );
		OutTotals.Add( TEXT("VertexColorMem"), FText::AsNumber( TotalEntry->VertexColorMem ) );
		OutTotals.Add( TEXT("InstVertexColorMem"), FText::AsNumber( TotalEntry->InstVertexColorMem ) );
		OutTotals.Add( TEXT("LightsLM"), FText::AsNumber( TotalEntry->LightsLM ) );
		OutTotals.Add( TEXT("LightsOther"), FText::AsNumber( TotalEntry->LightsOther ) );
		OutTotals.Add( TEXT("LightsTotal"), FText::AsNumber( TotalEntry->LightsTotal ) );
		OutTotals.Add( TEXT("ObjLightCost"), FText::AsNumber( TotalEntry->ObjLightCost ) );
		OutTotals.Add( TEXT("LightMapData"), FText::AsNumber( TotalEntry->LightMapData ) );
		OutTotals.Add( TEXT("LMSMResolution"), FText::AsNumber( TotalEntry->LMSMResolution ) );
		OutTotals.Add( TEXT("RadiusMin"), FText::AsNumber( TotalEntry->RadiusMin ) );
		OutTotals.Add( TEXT("RadiusMax"), FText::AsNumber( TotalEntry->RadiusMax ) );
		OutTotals.Add( TEXT("RadiusAvg"), FText::AsNumber( TotalEntry->RadiusAvg ) );
	}
}

void FPrimitiveStatsPage::OnEditorSelectionChanged( UObject* NewSelection, TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == PrimitiveObjectSets_SelectedObjects )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FPrimitiveStatsPage::OnEditorNewCurrentLevel( TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == PrimitiveObjectSets_CurrentLevel )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FPrimitiveStatsPage::OnShow( TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	// register delegates for scene changes we are interested in
	USelection::SelectionChangedEvent.AddRaw(this, &FPrimitiveStatsPage::OnEditorSelectionChanged, InParentStatsViewer);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FPrimitiveStatsPage::OnEditorNewCurrentLevel, InParentStatsViewer);
}

void FPrimitiveStatsPage::OnHide()
{
	// unregister delegates
	USelection::SelectionChangedEvent.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE