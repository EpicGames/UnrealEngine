// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStatsViewer.h"
#include "StatsPage.h"
#include "PrimitiveStats.h"

/** Stats page representing primitive component stats */
class FPrimitiveStatsPage : public FStatsPage<UPrimitiveStats>
{
public:
	/** Singleton accessor */
	static FPrimitiveStatsPage& Get();

	/** Begin IStatsPage interface */
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const override;
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const override;
	virtual void OnShow( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override;
	virtual void OnHide() override;
	/** End IStatsPage interface */

	virtual ~FPrimitiveStatsPage() = default;

private:

	/**
	 * Delegate to allow is to trigger a refresh on object selection 
	 * @param	NewSelection		The new selected object
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	void OnEditorSelectionChanged( UObject* NewSelection, TWeakPtr< class IStatsViewer > InParentStatsViewer );

	/** 
	 * Delegate to allow is to trigger a refresh on new level 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	void OnEditorNewCurrentLevel( TWeakPtr< class IStatsViewer > InParentStatsViewer );
};

