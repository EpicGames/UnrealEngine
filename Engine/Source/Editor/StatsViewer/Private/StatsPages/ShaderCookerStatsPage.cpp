// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/ShaderCookerStatsPage.h"
#include "Serialization/Csv/CsvParser.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIShaderFormatDefinitions.inl"
#include "StatsPages/ShaderCookerStatsPage.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer.ShaderCookerStats"

FShaderCookerStatsPage& FShaderCookerStatsPage::Get()
{
	static FShaderCookerStatsPage* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FShaderCookerStatsPage;
	}
	return *Instance;
}

class FShaderCookerStats
{
public:
	class FShaderCookerStatsSet
	{
	public:
		FString Name;
		TArray<TWeakObjectPtr<UShaderCookerStats>> Stats;
		bool bInitialized;

	};

	/** Singleton accessor */
	static FShaderCookerStats& Get();
	FShaderCookerStats();
	~FShaderCookerStats();


	TPair<FString, int32> FindCategory(FString Path)
	{
		for(TPair<FString, int32>& Pair : StatPatterns)
		{
			int32 Index = Path.Find(Pair.Key, ESearchCase::IgnoreCase);
			if(Index >= 0 && Index < 2)
			{
				return TPair<FString, int32>(StatCategoryNames[Pair.Value], Pair.Value);
			}
		}
		return TPair<FString, int32>(FString(""), 0);
	}
	TArray<FString> GetStatSetNames()
	{
		TArray<FString> Temp;
		for(FShaderCookerStatsSet& Set : StatSets)
		{
			Temp.Emplace(Set.Name);
		}
		return Temp;
	}

	FString GetStatSetName(int32 Index)
	{
		if((uint32)Index < NumSets())
		{
			return StatSets[Index].Name;
		}
		else
		{
			return "";
		}
	}

	const TArray<TWeakObjectPtr<UShaderCookerStats>>& GetShaderCookerStats(uint32 Index)
	{
		check(Index < (uint32)StatSets.Num());
		FShaderCookerStatsSet& Set = StatSets[Index];
		if(!Set.bInitialized)
		{
			Initialize(Index);
		}
		else if (!ensureMsgf(Set.Stats.IsEmpty() || Set.Stats[0].IsValid(), TEXT("Shader cook statistics set got stale. This should not happen and indicates an issue with refreshing FShaderCookerStatsPage!")))
		{
			Set.Stats.Empty();
			Initialize(Index);
		}
		return Set.Stats;
	}
	uint32 NumSets()
	{
		return StatSets.Num();
	}

	uint32 NumCategories()
	{
		return StatCategoryNames.Num();
	}
	FString GetCategoryName(int32 Index)
	{
		if(Index < StatCategoryNames.Num() && Index >= 0)
		{
			return StatCategoryNames[Index];
		}
		else
		{
			return FString("All");
		}
	}
	TArray<FShaderCookerStatsSet> StatSets;
	TArray<FString> StatCategoryNames;
	TArray< TPair<FString, int32> > StatPatterns;

	void Initialize(uint32 Index);
};

void FShaderCookerStats::Initialize(uint32 Index)
{
	TArray<FString> PlatformNames;
	PlatformNames.Reserve(SP_NumPlatforms);

	for (int32 PlatformIndex = 0; PlatformIndex < SP_NumPlatforms; ++PlatformIndex)
	{
		const EShaderPlatform Platform = static_cast<EShaderPlatform>(PlatformIndex);

		const FName ShaderFormatName = FDataDrivenShaderPlatformInfo::IsValid(Platform)
			? FDataDrivenShaderPlatformInfo::GetShaderFormat(Platform) : NAME_None;

		if (ShaderFormatName != NAME_None)
		{
			FString FormatName = ShaderFormatName.ToString();
			if (FormatName.StartsWith(TEXT("SF_")))
			{
				FormatName.MidInline(3, MAX_int32, EAllowShrinking::No);
			}
			PlatformNames.Add(MoveTemp(FormatName));
		}
		else
		{
			PlatformNames.Add(TEXT("unknown"));
		}
	}
	FShaderCookerStatsSet& Set = StatSets[Index];
	FString CSVData;
	if (FFileHelper::LoadFileToString(CSVData, *Set.Name))
	{
		FCsvParser CsvParser(CSVData);
		const FCsvParser::FRows& Rows = CsvParser.GetRows();
		int32 RowIndex = 0;
		int32 IndexPath = -1;
		int32 IndexPlatform = -1;
		int32 IndexCompiled = -1;
		int32 IndexCooked = -1;
		int32 IndexPermutations = -1;
		int32 IndexCompileTime = -1;
		for (const TArray<const TCHAR*>& Row : Rows)
		{
			if (RowIndex == 0)
			{
				for (int32 Column = 0; Column < Row.Num(); ++Column)
				{
					FString Key = Row[Column];
					if (Key == TEXT("Path"))
					{
						IndexPath = Column;
					}
					else if (Key == TEXT("Name"))
					{
						if (IndexPath == -1)
						{
							IndexPath = Column;
						}
					}
					else if (Key == TEXT("Platform"))
					{
						IndexPlatform = Column;
					}
					else if (Key == TEXT("Compiled"))
					{
						IndexCompiled = Column;
					}
					else if (Key == TEXT("Cooked"))
					{
						IndexCooked = Column;
					}
					else if (Key == TEXT("Permutations"))
					{
						IndexPermutations = Column;
					}
					else if (Key == TEXT("Compiletime"))
					{
						IndexCompileTime = Column;
					}
				}
			}
			else
			{
				FString Path = IndexPath >= 0 && IndexPath < Row.Num() ? Row[IndexPath] : TEXT("?");
#define GET_INT(Index) (Index >= 0 && Index < Row.Num() ? FCString::Atoi(Row[Index]) : 424242)
#define GET_FLOAT(Index) (Index >= 0 && Index < Row.Num() ? FCString::Atof(Row[Index]) : 0.f)
				int32 Platform = GET_INT(IndexPlatform);
				int32 Compiled = GET_INT(IndexCompiled);
				int32 Cooked = GET_INT(IndexCooked);
				int32 Permutations = GET_INT(IndexPermutations);
				float CompileTime = GET_FLOAT(IndexCompileTime);
#undef GET_INT
#undef GET_FLOAT


				UShaderCookerStats* Stat = NewObject<UShaderCookerStats>();

				int32 LastSlash = -1;
				int32 LastDot = -1;
				FString Name = Path;
				if (Path.FindLastChar('/', LastSlash) && Path.FindLastChar('.', LastDot))
				{
					Name = Path.Mid(LastSlash + 1, LastDot - LastSlash - 1);
				}
				Stat->Name = Name;
				Stat->Path = Path;
				Stat->Platform = Platform < PlatformNames.Num() ? PlatformNames[Platform] : TEXT("unknown");
				Stat->Compiled = Compiled;
				Stat->Cooked = Cooked;
				Stat->Permutations = Permutations;
				Stat->CompileTime = CompileTime;
				TPair<FString, int32> Category = FindCategory(Path);
				Stat->Category = Category.Key;
				Stat->CategoryIndex = Category.Value;
				Set.Stats.Emplace(Stat);
			}
			RowIndex++;
		}
	}
	Set.bInitialized = true;
}


FShaderCookerStats& FShaderCookerStats::Get()
{
	static FShaderCookerStats* Instance = NULL;
	if (Instance == NULL)
	{
		Instance = new FShaderCookerStats;
	}
	return *Instance;
}

FShaderCookerStats::FShaderCookerStats()
{
	TArray<FString> Files;
	FString BasePath = FString::Printf(TEXT("%s/MaterialStats/"), *FPaths::ProjectSavedDir());
	FPlatformFileManager::Get().GetPlatformFile().FindFiles(Files, *BasePath, TEXT("csv"));

	FString MirrorLocation;
	GConfig->GetString(TEXT("/Script/Engine.ShaderCompilerStats"), TEXT("MaterialStatsLocation"), MirrorLocation, GGameIni);
	FParse::Value(FCommandLine::Get(), TEXT("MaterialStatsMirror="), MirrorLocation);
	if (!MirrorLocation.IsEmpty())
	{
		TArray<FString> RemoteFiles;
		FString RemotePath = FPaths::Combine(*MirrorLocation, FApp::GetProjectName(), *FApp::GetBranchName());
		FPlatformFileManager::Get().GetPlatformFile().FindFiles(RemoteFiles, *RemotePath, TEXT("csv"));
		Files.Append(RemoteFiles);
	}

	for (FString Filename : Files)
	{
		FShaderCookerStatsSet Set;
		Set.Name = Filename;
		Set.bInitialized = false;
		StatSets.Emplace(Set);
	}

	TMap<FString, int32> CategoryToIndex;
	StatCategoryNames.Add("*All*");

	auto LoadCategories =[this, &CategoryToIndex](FString Path)
	{
		FString CSVData;
		if(FFileHelper::LoadFileToString(CSVData, *Path))
		{
			FCsvParser CsvParser(CSVData);
			const FCsvParser::FRows& Rows = CsvParser.GetRows();
			for (const TArray<const TCHAR*>& Row : Rows)
			{
				if (Row.Num() != 2)
				{
					continue;
				}
				FString Category = Row[0];
				FString Pattern = Row[1];
				Category = Category.TrimStart().TrimEnd().TrimQuotes();
				Pattern = Pattern.TrimStart().TrimEnd().TrimQuotes();
				//allow comments 
				if (Category[0] == TCHAR('#') || Category[0] == TCHAR(';'))
				{
					continue;
				}
				if (Pattern[0] == TCHAR('\\') || Pattern[0] == TCHAR('/'))
				{
					Pattern = Pattern.Mid(1);
				}

				int32 Index = -1;
				if (CategoryToIndex.Contains(Category))
				{
					Index = CategoryToIndex[Category];
				}
				else
				{
					Index = StatCategoryNames.Num();
					CategoryToIndex.FindOrAdd(Category) = Index;
					StatCategoryNames.Add(Category);
				}
				TPair<FString, int32> Pair(Pattern, Index);
				StatPatterns.Add(Pair);
			}
		}
	};

	//load from both engine & project
	LoadCategories(FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("ShaderCategories.csv")));
	LoadCategories(FPaths::Combine(FPaths::EngineConfigDir(), TEXT("ShaderCategories.csv")));
}

FShaderCookerStats::~FShaderCookerStats()
{

}

TSharedPtr<SWidget> FShaderCookerStatsPage::GetCustomWidget(TWeakPtr<IStatsViewer> InParentStatsViewer)
{
	if (!CustomWidget.IsValid())
	{
		SAssignNew(CustomWidget, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(PlatformComboButton, SComboButton)
				.ContentPadding(3.f)
				.OnGetMenuContent(this, &FShaderCookerStatsPage::OnGetPlatformButtonMenuContent, InParentStatsViewer)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FShaderCookerStatsPage::OnGetPlatformMenuLabel)
					.ToolTipText(LOCTEXT("Platform_ToolTip", "Platform"))
				]
			];
	}

	return CustomWidget;
}

TSharedRef<SWidget> FShaderCookerStatsPage::OnGetPlatformButtonMenuContent(TWeakPtr<IStatsViewer> InParentStatsViewer) const
{
	FMenuBuilder MenuBuilder(true, NULL);
	FShaderCookerStats& Stats = FShaderCookerStats::Get();
	for (int32 Index = 0; Index < (int32)Stats.NumSets(); ++Index)
	{
		FString Name = Stats.GetStatSetName(Index);
		FText MenuText = FText::FromString(Name);

		MenuBuilder.AddMenuEntry(
			MenuText,
			MenuText,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<FShaderCookerStatsPage*>(this), &FShaderCookerStatsPage::OnPlatformClicked, InParentStatsViewer, Index),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FShaderCookerStatsPage::IsPlatformSetSelected, Index)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	return MenuBuilder.MakeWidget();
}

void FShaderCookerStatsPage::OnPlatformClicked(TWeakPtr<IStatsViewer> InParentStatsViewer, int32 Index)
{
	bool bChanged = SelectedPlatform != Index;
	SelectedPlatform = Index;
	if(bChanged)
	{
		InParentStatsViewer.Pin()->Refresh();
	}
}

bool FShaderCookerStatsPage::IsPlatformSetSelected(int32 Index) const
{
	return SelectedPlatform == Index;
}

FText FShaderCookerStatsPage::OnGetPlatformMenuLabel() const
{
	FString ActiveSetName = FShaderCookerStats::Get().GetStatSetName(SelectedPlatform);
	FText Text = FText::FromString(ActiveSetName);
	return Text;
}

void FShaderCookerStatsPage::Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const
{
	FShaderCookerStats& Stats = FShaderCookerStats::Get();
	if((uint32)SelectedPlatform < Stats.NumSets())
	{
		const TArray<TWeakObjectPtr<UShaderCookerStats>>& CookStats = Stats.GetShaderCookerStats(SelectedPlatform);
		for(const TWeakObjectPtr<UShaderCookerStats>& Stat : CookStats)
		{
			if (Stat.IsValid())
			{
				if(0 == ObjectSetIndex || ObjectSetIndex == Stat->CategoryIndex)
				{
					OutObjects.Add(Stat);
				}
			}
		}
	}
}

void FShaderCookerStatsPage::GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if(InObjects.Num())
	{
		UShaderCookerStats* TotalEntry = NewObject<UShaderCookerStats>();

		for( auto It = InObjects.CreateConstIterator(); It; ++It )
		{
			UShaderCookerStats* StatsEntry = Cast<UShaderCookerStats>( It->Get() );
			TotalEntry->Compiled += StatsEntry->Compiled;
			TotalEntry->Cooked += StatsEntry->Cooked;
			TotalEntry->Permutations += StatsEntry->Permutations;
			TotalEntry->CompileTime += StatsEntry->CompileTime;
		}

		OutTotals.Add( TEXT("Compiled"), FText::AsNumber( TotalEntry->Compiled) );
		OutTotals.Add( TEXT("Cooked"), FText::AsNumber( TotalEntry->Cooked) );
		OutTotals.Add( TEXT("CompileTime"), FText::AsNumber( TotalEntry->CompileTime ));
		OutTotals.Add( TEXT("Permutations"), FText::AsNumber( TotalEntry->Permutations) );
	}
}
void FShaderCookerStatsPage::OnShow( TWeakPtr< IStatsViewer > InParentStatsViewer )
{
}

void FShaderCookerStatsPage::OnHide()
{
}

int32 FShaderCookerStatsPage::GetObjectSetCount() const
{
	return FShaderCookerStats::Get().NumCategories();
}
FString FShaderCookerStatsPage::GetObjectSetName(int32 InObjectSetIndex) const
{
	return FShaderCookerStats::Get().GetCategoryName(InObjectSetIndex);
}
FString FShaderCookerStatsPage::GetObjectSetToolTip(int32 InObjectSetIndex) const 
{
	return GetObjectSetName(InObjectSetIndex);
}


#undef LOCTEXT_NAMESPACE
