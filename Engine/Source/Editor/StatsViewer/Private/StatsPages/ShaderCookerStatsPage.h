// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStatsViewer.h"
#include "StatsPage.h"
#include "ShaderCookerStats.h"

class SComboButton;
class SButton;



/** Stats page representing material cooking stats */
class FShaderCookerStatsPage : public FStatsPage<UShaderCookerStats>, public TSharedFromThis<FShaderCookerStatsPage>
{
public:
	/** Singleton accessor */
	static FShaderCookerStatsPage& Get();

	/** Begin IStatsPage interface */
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const override;
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const override;
	virtual TSharedPtr<SWidget> GetCustomWidget(TWeakPtr< class IStatsViewer > InParentStatsViewer) override;
	virtual void OnShow( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override;
	virtual void OnHide() override;
	virtual int32 GetObjectSetCount() const override;;
	virtual FString GetObjectSetName(int32 InObjectSetIndex) const override;
	virtual FString GetObjectSetToolTip(int32 InObjectSetIndex) const override;

	/** End IStatsPage interface */

	virtual ~FShaderCookerStatsPage() = default;

private:
	/** Generate label for the displayed objects combobox */
	FText OnGetPlatformMenuLabel() const;
	void OnPlatformClicked(TWeakPtr<IStatsViewer> InParentStatsViewer, int32 Index);
	bool IsPlatformSetSelected(int32 Index) const;
	TSharedRef<SWidget> OnGetPlatformButtonMenuContent(TWeakPtr< class IStatsViewer > InParentStatsViewer) const;

	/** Custom widget for this page */
	TSharedPtr<SWidget> CustomWidget;

	/** Platform combo button */
	TSharedPtr<SComboButton> PlatformComboButton;
	
	/** Current Platform */
	int32 SelectedPlatform = 0;

};

