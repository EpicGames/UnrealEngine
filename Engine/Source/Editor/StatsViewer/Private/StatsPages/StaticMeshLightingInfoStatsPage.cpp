// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/StaticMeshLightingInfoStatsPage.h"
#include "GameFramework/Actor.h"
#include "Engine/GameViewportClient.h"
#include "Components/StaticMeshComponent.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Components/LightComponent.h"
#include "Editor.h"
#include "IPropertyTableRow.h"
#include "BusyCursor.h"
#include "ScopedTransaction.h"
#include "StaticMeshComponentLODInfo.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Engine/LevelStreaming.h"
#include "Engine/MapBuildDataRegistry.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer.StaticMeshLightingInfo"

FStaticMeshLightingInfoStatsPage& FStaticMeshLightingInfoStatsPage::Get()
{
	static FStaticMeshLightingInfoStatsPage* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FStaticMeshLightingInfoStatsPage;
	}
	return *Instance;
}

/** Helper class to generate statistics */
struct StaticMeshLightingInfoStatsGenerator
{
	StaticMeshLightingInfoStatsGenerator(UWorld* InWorld)
	{
		World = InWorld != nullptr ? InWorld : GWorld;
	}

	UWorld* World;

	/** The lights in the world which the system is scanning. */
	TArray<ULightComponent*> AllLights;

	void AddRequiredLevels( EStaticMeshLightingInfoObjectSets InObjectSet, UWorld* InWorld, TArray<ULevel*>& OutLevels )
	{
		switch (InObjectSet)
		{ 
		case StaticMeshLightingInfoObjectSets_CurrentLevel:
			{
				check(InWorld);
				OutLevels.AddUnique(InWorld->GetCurrentLevel());
			}
			break;
		case StaticMeshLightingInfoObjectSets_SelectedLevels:
			{
				TArray<TObjectPtr<class ULevel>>& SelectedLevels = InWorld->GetSelectedLevels();
				for(auto It = SelectedLevels.CreateIterator(); It; ++It)
				{
					ULevel* Level = *It;
					if (Level)
					{
						OutLevels.AddUnique(Level);
					}
				}

				if (OutLevels.Num() == 0)
				{
					// Fall to the current level...
					check(InWorld);
					OutLevels.AddUnique(InWorld->GetCurrentLevel());
				}
			}
			break;
		case StaticMeshLightingInfoObjectSets_AllLevels:
			{
				if (InWorld != NULL)
				{
					// Add main level.
					OutLevels.AddUnique(InWorld->PersistentLevel);

					// Add secondary levels.
					for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
					{
						if (StreamingLevel)
						{
							if (ULevel* Level = StreamingLevel->GetLoadedLevel())
							{
								OutLevels.AddUnique( Level );
							}
						}
					}
				}
			}
			break;
		}
	}

	/** Add an item to the output objects array */
	void AddItem(UStaticMeshComponent* InComponent, AActor* InActor, TArray< TWeakObjectPtr<UObject> >& OutObjects)
	{
		int32 TextureLightMapMemoryUsage;
		int32 TextureShadowMapMemoryUsage;
		int32 VertexLightMapMemoryUsage;
		int32 VertexShadowMapMemoryUsage;
		int32 StaticLightingResolution;
		bool bTextureMapping;
		bool bHasLightmapTexCoords;

		if(InComponent != NULL && InComponent->GetEstimatedLightAndShadowMapMemoryUsage(
				TextureLightMapMemoryUsage, 
				TextureShadowMapMemoryUsage,
				VertexLightMapMemoryUsage, 
				VertexShadowMapMemoryUsage,
				StaticLightingResolution, 
				bTextureMapping, 
				bHasLightmapTexCoords) )
		{
			UStaticMeshLightingInfo* Entry = NewObject<UStaticMeshLightingInfo>();
			Entry->AddToRoot();
			OutObjects.Add(Entry);

			Entry->StaticMeshActor = InActor;
			Entry->StaticMeshComponent = InComponent;
			Entry->StaticMesh = InComponent->GetStaticMesh();
			
			// Show all of the lightmap texture names so we can correlate meshes with the actual texture data
			// for debugging encoding concerns. Only do this if we have static lighting enabled/built (LODData exists)
			if (InComponent->LODData.Num())
			{
				const FMeshMapBuildData* MeshMapBuildData = InComponent->GetMeshMapBuildData(InComponent->LODData[0]);
				if (MeshMapBuildData && MeshMapBuildData->LightMap)
				{
					TArray<UTexture2D*> Textures;
					FLightMap2D* LightMap2D = MeshMapBuildData->LightMap->GetLightMap2D();
					if (LightMap2D)
					{
						LightMap2D->GetReferencedTextures(Textures);
					}
					for (UTexture2D* Texture : Textures)
					{
						Entry->LightmapTextureNames.Add(Texture->GetName());
					}
				}
			}

			Entry->TextureLightMapMemoryUsage = (float)TextureLightMapMemoryUsage / 1024.0f;
			Entry->TextureShadowMapMemoryUsage = (float)TextureShadowMapMemoryUsage / 1024.0f;
			Entry->VertexLightMapMemoryUsage = (float)VertexLightMapMemoryUsage / 1024.0f;
			Entry->VertexShadowMapMemoryUsage = (float)VertexShadowMapMemoryUsage / 1024.0f;
			Entry->StaticLightingResolution = StaticLightingResolution;
			Entry->bTextureMapping = bTextureMapping;
			Entry->bHasLightmapTexCoords = bHasLightmapTexCoords;

			Entry->UpdateNames();

			// Find the lights relevant to the primitive.
			TArray<ULightComponent*> LightMapRelevantLights;
			TArray<ULightComponent*> ShadowMapRelevantLights;
			for (int32 LightIndex = 0; LightIndex < AllLights.Num(); LightIndex++)
			{
				ULightComponent* Light = AllLights[LightIndex];
				// Only add enabled lights
				if (Light->GetVisibleFlag() && Light->AffectsPrimitive(InComponent))
				{
					// Check whether the light should use a light-map or shadow-map.
					const bool bHasStaticLighting = Light->HasStaticLighting();
					if (bHasStaticLighting)
					{
						LightMapRelevantLights.Add(Light);
					}
					// only allow for shadow maps if shadow casting is enabled
					else if (Light->CastShadows && Light->CastStaticShadows)
					{
						ShadowMapRelevantLights.Add(Light);
					}
				}
			}

			Entry->LightMapLightCount = LightMapRelevantLights.Num();
			Entry->ShadowMapLightCount = ShadowMapRelevantLights.Num();
		}
	}

	/** Add items to the output object array according to the input object set */
	void Generate(EStaticMeshLightingInfoObjectSets InObjectSet, TArray< TWeakObjectPtr<UObject> >& OutObjects)
	{
		/** The levels we are gathering information for. */
		TArray<ULevel*> Levels;

		// Fill the light list
		for (TObjectIterator<ULightComponent> LightIt; LightIt; ++LightIt)
		{
			ULightComponent* const Light = *LightIt;

			const bool bLightIsInWorld = Light->GetOwner() && !Light->GetOwner()->HasAnyFlags(RF_ClassDefaultObject) && World->ContainsActor(Light->GetOwner());
			if (bLightIsInWorld)
			{
				if (Light->HasStaticLighting() || Light->HasStaticShadowing())
				{
					// Add the light to the system's list of lights in the world.
					AllLights.Add(Light);
				}

				//append to the list of levels in use
				AddRequiredLevels( InObjectSet, World, Levels );				
			}
		}

		if (Levels.Num() > 0)
		{
			// Iterate over static mesh components in the list of levels...
			for (TObjectIterator<UStaticMeshComponent> SMCIt; SMCIt; ++SMCIt)
			{
				AActor* Owner = (*SMCIt)->GetOwner();
				if (Owner && !Owner->HasAnyFlags(RF_ClassDefaultObject) )
				{
					int32 Dummy;
					ULevel* CheckLevel = Owner->GetLevel();
					if ((CheckLevel != NULL) && (Levels.Find(CheckLevel, Dummy)))
					{
						AddItem(*SMCIt, Owner, OutObjects);
					}
				}
			}
		}
	}
};

void FStaticMeshLightingInfoStatsPage::Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const
{
	StaticMeshLightingInfoStatsGenerator Generator( GetWorld() );
	Generator.Generate((EStaticMeshLightingInfoObjectSets)ObjectSetIndex, OutObjects);
}

void FStaticMeshLightingInfoStatsPage::GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if(InObjects.Num())
	{
		UStaticMeshLightingInfo* TotalEntry = NewObject<UStaticMeshLightingInfo>();

		for( auto It = InObjects.CreateConstIterator(); It; ++It )
		{
			UStaticMeshLightingInfo* StatsEntry = Cast<UStaticMeshLightingInfo>( It->Get() );
			
			TotalEntry->LightMapLightCount += StatsEntry->LightMapLightCount;
			TotalEntry->TextureLightMapMemoryUsage += StatsEntry->TextureLightMapMemoryUsage;
			TotalEntry->VertexLightMapMemoryUsage += StatsEntry->VertexLightMapMemoryUsage;
			TotalEntry->ShadowMapLightCount += StatsEntry->ShadowMapLightCount;
			TotalEntry->TextureShadowMapMemoryUsage += StatsEntry->TextureShadowMapMemoryUsage;
			TotalEntry->VertexShadowMapMemoryUsage += StatsEntry->VertexShadowMapMemoryUsage;
		}

		OutTotals.Add( TEXT("LightMapLightCount"), FText::AsNumber( TotalEntry->LightMapLightCount ) );
		OutTotals.Add( TEXT("TextureLightMapMemoryUsage"), FText::AsNumber( TotalEntry->TextureLightMapMemoryUsage ) );
		OutTotals.Add( TEXT("VertexLightMapMemoryUsage"), FText::AsNumber( TotalEntry->VertexLightMapMemoryUsage ) );
		OutTotals.Add( TEXT("ShadowMapLightCount"), FText::AsNumber( TotalEntry->ShadowMapLightCount ) );
		OutTotals.Add( TEXT("TextureShadowMapMemoryUsage"), FText::AsNumber( TotalEntry->TextureShadowMapMemoryUsage ) );
		OutTotals.Add( TEXT("VertexShadowMapMemoryUsage"), FText::AsNumber( TotalEntry->VertexShadowMapMemoryUsage ) );
	}
}

TSharedPtr<SWidget> FStaticMeshLightingInfoStatsPage::GetCustomWidget(TWeakPtr<IStatsViewer> InParentStatsViewer)
{
	if( !CustomWidget.IsValid() )
	{
		SAssignNew(CustomWidget, SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth() 
		.Padding(0.0f) 
		.HAlign(HAlign_Fill)
		[
			SAssignNew(SwapComboButton, SComboButton)
			.ContentPadding(3.f)
			.OnGetMenuContent( this, &FStaticMeshLightingInfoStatsPage::OnGetSwapComboButtonMenuContent, InParentStatsViewer )
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(LOCTEXT("Swap", "Swap"))
				.ToolTipText(LOCTEXT("SwapObjectToolTip", "Swap selected objects between Vertex and Texture lighting"))
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f) 
		.HAlign(HAlign_Fill)
		[
			SAssignNew(SetToComboButton, SComboButton)
			.ContentPadding(3.f)
			.OnGetMenuContent( this, &FStaticMeshLightingInfoStatsPage::OnGetSetToComboButtonMenuContent, InParentStatsViewer )
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(LOCTEXT("SetTo", "SetTo"))
				.ToolTipText(LOCTEXT("SetToToolTip", "Set selected objects to either Vertex or Texture lighting"))
			]
		];
	}

	return CustomWidget;
}

void FStaticMeshLightingInfoStatsPage::SwapMappingMethodOnSelectedComponents(TWeakPtr<IStatsViewer> InParentStatsViewer, int32 InStaticLightingResolution) const
{
	if(InParentStatsViewer.IsValid())
	{
		const FScopedBusyCursor BusyCursor;

		// first get the selected objects in the table
		TArray<UStaticMeshLightingInfo*> SelectedObjects;
		TSharedPtr<IPropertyTable> PropertyTable = InParentStatsViewer.Pin()->GetPropertyTable();
		const TSet< TSharedRef<IPropertyTableRow> >& Rows = PropertyTable->GetSelectedRows();
		for(auto RowIt = Rows.CreateConstIterator(); RowIt; ++RowIt)
		{
			TSharedRef<IPropertyTableRow> Row = *RowIt;
			UStaticMeshLightingInfo* Entry = Cast<UStaticMeshLightingInfo>( Row->GetDataSource()->AsUObject().Get() );
			SelectedObjects.Add(Entry);
		}

		if (SelectedObjects.Num() > 0)
		{
			const FScopedTransaction Transaction( LOCTEXT("StaticMeshLightingInfoSwap", "DlgStaticMeshLightingInfo:Swap") );

			bool bNeedsRefresh = false;
			for (int32 CompIdx = 0; CompIdx < SelectedObjects.Num(); CompIdx++)
			{
				UStaticMeshLightingInfo* Entry = SelectedObjects[CompIdx];
				if (Entry->StaticMeshActor.IsValid())
				{
					Entry->StaticMeshActor->Modify();
				}
				if ( Entry->StaticMeshComponent.IsValid())
				{
					Entry->StaticMeshComponent->Modify();
					Entry->StaticMeshComponent->SetStaticLightingMapping(!(Entry->bTextureMapping), InStaticLightingResolution);
					Entry->StaticMeshComponent->InvalidateLightingCache();
					Entry->StaticMeshComponent->ReregisterComponent();
				}

				bNeedsRefresh = true;
			}

			if (bNeedsRefresh)
			{
				InParentStatsViewer.Pin()->Refresh();
			}
		}
	}
}

void FStaticMeshLightingInfoStatsPage::OnSwapClicked(TWeakPtr<IStatsViewer> InParentStatsViewer, ESwapOptions::Type InSwapOption) const
{
	switch (InSwapOption)
	{
	case ESwapOptions::Swap:
		{
			SwapMappingMethodOnSelectedComponents(InParentStatsViewer, 0);
		}
		break;
	case ESwapOptions::SwapAskRes:
		{
			GetUserSetStaticLightmapResolution(InParentStatsViewer, true);
		}
		break;
	}
}

TSharedRef<SWidget> FStaticMeshLightingInfoStatsPage::OnGetSwapComboButtonMenuContent(TWeakPtr<IStatsViewer> InParentStatsViewer) const
{
	FMenuBuilder MenuBuilder( true, NULL );

	MenuBuilder.AddMenuEntry( 
		LOCTEXT("Swap", "Swap"), 
		LOCTEXT("SwapToolTip", "Swap between Vertex and Texture"), 
		FSlateIcon(), 
		FUIAction( FExecuteAction::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnSwapClicked, InParentStatsViewer, ESwapOptions::Swap ) ) );

	MenuBuilder.AddMenuEntry( 
		LOCTEXT("SwapAskRes", "Swap(Res)..."),
		LOCTEXT("SwapAskResToolTip", "Swap between Vertex and Texture, prompt for Resolution"), 
		FSlateIcon(), 
		FUIAction( FExecuteAction::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnSwapClicked, InParentStatsViewer, ESwapOptions::Swap ) ) );

	return MenuBuilder.MakeWidget();
}

void FStaticMeshLightingInfoStatsPage::SetMappingMethodOnSelectedComponents(TWeakPtr<IStatsViewer> InParentStatsViewer, bool bInTextureMapping, int32 InStaticLightingResolution) const
{
	if(InParentStatsViewer.IsValid())
	{
		const FScopedBusyCursor BusyCursor;

		// first get the selected objects in the table
		TArray<UStaticMeshLightingInfo*> SelectedObjects;
		TSharedPtr<IPropertyTable> PropertyTable = InParentStatsViewer.Pin()->GetPropertyTable();
		const TSet< TSharedRef<IPropertyTableRow> >& Rows = PropertyTable->GetSelectedRows();
		for(auto RowIt = Rows.CreateConstIterator(); RowIt; ++RowIt)
		{
			TSharedRef<IPropertyTableRow> Row = *RowIt;
			UStaticMeshLightingInfo* Entry = Cast<UStaticMeshLightingInfo>( Row->GetDataSource()->AsUObject().Get() );
			SelectedObjects.Add(Entry);
		}

		if (SelectedObjects.Num() > 0)
		{
			const FScopedTransaction Transaction( LOCTEXT("StaticMeshLightingInfoSet", "DlgStaticMeshLightingInfo:Set") );

			bool bNeedsRefresh = false;
			for (int32 CompIdx = 0; CompIdx < SelectedObjects.Num(); CompIdx++)
			{
				UStaticMeshLightingInfo* Entry = SelectedObjects[CompIdx];
				if (Entry->StaticMeshActor.IsValid())
				{
					Entry->StaticMeshActor->Modify();
				}
				if ( Entry->StaticMeshComponent.IsValid())
				{
					Entry->StaticMeshComponent->Modify();
					Entry->StaticMeshComponent->SetStaticLightingMapping(bInTextureMapping, InStaticLightingResolution);
					Entry->StaticMeshComponent->InvalidateLightingCache();
					Entry->StaticMeshComponent->ReregisterComponent();
				}
			
				bNeedsRefresh = true;
			}

			if (bNeedsRefresh)
			{
				InParentStatsViewer.Pin()->Refresh();
			}
		}
	}
}

void FStaticMeshLightingInfoStatsPage::OnSetToClicked(TWeakPtr<IStatsViewer> InParentStatsViewer, ESetToOptions::Type InSetToOption) const
{
	switch (InSetToOption)
	{
	case ESetToOptions::Vertex:
		{
			SetMappingMethodOnSelectedComponents(InParentStatsViewer, false, 0);
		}
		break;
	case ESetToOptions::Texture:
		{
			SetMappingMethodOnSelectedComponents(InParentStatsViewer, true, 0);
		}
		break;
	case ESetToOptions::TextureAskRes:
		{
			GetUserSetStaticLightmapResolution(InParentStatsViewer, false);
		}
		break;
	}
}

TSharedRef<SWidget> FStaticMeshLightingInfoStatsPage::OnGetSetToComboButtonMenuContent(TWeakPtr<IStatsViewer> InParentStatsViewer) const
{
	FMenuBuilder MenuBuilder( true, NULL );

	MenuBuilder.AddMenuEntry( 
		LOCTEXT("SetToVertexMapping", "Set To Vertex"), 
		LOCTEXT("SetToVertexMappingToolTip", "Set to Vertex"), 
		FSlateIcon(), 
		FUIAction( FExecuteAction::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnSetToClicked, InParentStatsViewer, ESetToOptions::Vertex ) ) );

	MenuBuilder.AddMenuEntry( 
		LOCTEXT("SetToTextureMapping", "Set To Texture"), 
		LOCTEXT("SetToTextureMappingToolTip", "Set to Texture"), 
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnSetToClicked, InParentStatsViewer, ESetToOptions::Vertex ) ) );

	MenuBuilder.AddMenuEntry( 
		LOCTEXT("SetToTextureMappingAskRes", "Set To Texture(Res)..."),
		LOCTEXT("SetToTextureMappingAskResToolTip", "Set To Texture, prompt for Resolution"),
		FSlateIcon(), 
		FUIAction( FExecuteAction::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnSetToClicked, InParentStatsViewer, ESetToOptions::TextureAskRes ) ) );

	return MenuBuilder.MakeWidget();
}

void FStaticMeshLightingInfoStatsPage::OnResolutionCommitted(const FText& ResolutionText, ETextCommit::Type CommitInfo, TWeakPtr<IStatsViewer> InParentStatsViewer, bool bSwap) const
{
	// If we have a valid resolution
	int32 NewResolution = INDEX_NONE;
	if ( CommitInfo == ETextCommit::OnEnter && !ResolutionText.IsEmpty() && ResolutionText.IsNumeric() )
	{
		NewResolution = FCString::Atoi( *ResolutionText.ToString() );
		if (NewResolution >= 0)
		{
			NewResolution = FMath::Max(((NewResolution + 3) & ~3),4);
		}
	}

	// Remove the popup window
	if (ResolutionEntryMenu.IsValid())
	{
		ResolutionEntryMenu.Pin()->Dismiss();
	}

	// Do we have a valid resolution to use
	if (NewResolution != INDEX_NONE)
	{
		// Are we wanting to do a swap ex or a setto ex?
		if ( bSwap )
		{
			SwapMappingMethodOnSelectedComponents(InParentStatsViewer, NewResolution);
		}
		else
		{
			SetMappingMethodOnSelectedComponents(InParentStatsViewer, true, NewResolution);
		}
	}
}

void FStaticMeshLightingInfoStatsPage::GetUserSetStaticLightmapResolution(TWeakPtr<IStatsViewer> InParentStatsViewer, bool bSwap) const
{
	if(InParentStatsViewer.IsValid())
	{
		int32 DefaultRes = 0;
		verify(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("DefaultStaticMeshLightingRes"), DefaultRes, GLightmassIni));

		TSharedRef<STextEntryPopup> TextEntry = 
			SNew(STextEntryPopup)
			.Label(LOCTEXT("StaticMeshLightingInfo_GetResolutionTitle", "Enter Lightmap Resolution"))
			.HintText(LOCTEXT("StaticMeshLightingInfo_GetResolutionToolTip", "Will round to power of two"))
			.DefaultText(FText::AsNumber(DefaultRes))
			.OnTextCommitted( FOnTextCommitted::CreateSP( this, &FStaticMeshLightingInfoStatsPage::OnResolutionCommitted, InParentStatsViewer, bSwap ) )
			.ClearKeyboardFocusOnCommit( false );

		ResolutionEntryMenu = FSlateApplication::Get().PushMenu(
			InParentStatsViewer.Pin().ToSharedRef(),
			FWidgetPath(),
			TextEntry,
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
			);
	}
}

void FStaticMeshLightingInfoStatsPage::OnEditorNewCurrentLevel( TWeakPtr<IStatsViewer> InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == StaticMeshLightingInfoObjectSets_CurrentLevel )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FStaticMeshLightingInfoStatsPage::OnEditorLevelSelected( TWeakPtr<IStatsViewer> InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == StaticMeshLightingInfoObjectSets_SelectedLevels )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FStaticMeshLightingInfoStatsPage::OnShow( TWeakPtr<IStatsViewer> InParentStatsViewer )
{
	// register delegates for scene changes we are interested in
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FStaticMeshLightingInfoStatsPage::OnEditorNewCurrentLevel, InParentStatsViewer);
	GWorld->OnSelectedLevelsChanged().AddSP( this, &FStaticMeshLightingInfoStatsPage::OnEditorLevelSelected, InParentStatsViewer );
}

void FStaticMeshLightingInfoStatsPage::OnHide()
{
	// unregister delegates
	GWorld->OnSelectedLevelsChanged().RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE
