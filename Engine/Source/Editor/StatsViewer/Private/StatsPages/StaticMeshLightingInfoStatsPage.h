// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IStatsViewer.h"
#include "Framework/Application/IMenu.h"
#include "StatsPage.h"
#include "StaticMeshLightingInfo.h"

class SComboButton;

struct ESwapOptions
{
	enum Type
	{
		Swap = 0,
		SwapAskRes
	};
};

struct ESetToOptions
{
	enum Type
	{
		Vertex = 0,
		Texture,
		TextureAskRes
	};
};

/** Stats page representing static mesh lighting info */
class FStaticMeshLightingInfoStatsPage : public FStatsPage<UStaticMeshLightingInfo>, public TSharedFromThis<FStaticMeshLightingInfoStatsPage>
{
public:
	/** Singleton accessor */
	static FStaticMeshLightingInfoStatsPage& Get();

	/** Begin IStatsPage interface */
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const override;
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const override;
	virtual TSharedPtr<SWidget> GetCustomWidget(TWeakPtr< class IStatsViewer > InParentStatsViewer) override;
	virtual void OnShow( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override;
	virtual void OnHide() override;
	/** End IStatsPage interface */

	virtual ~FStaticMeshLightingInfoStatsPage() = default;

private:

	/** 
	 * Get the content for the swap combo button menu 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	TSharedRef<SWidget> OnGetSwapComboButtonMenuContent(TWeakPtr< class IStatsViewer > InParentStatsViewer) const;

	/** 
	 * Get the content for the 'set to' combo button menu 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	TSharedRef<SWidget> OnGetSetToComboButtonMenuContent(TWeakPtr< class IStatsViewer > InParentStatsViewer) const;

	/** 
	 * Swap button was clicked 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 * @param	InSwapOption		The type to swap to
	 */
	void OnSwapClicked(TWeakPtr< class IStatsViewer > InParentStatsViewer, ESwapOptions::Type InSwapOption) const;

	/** 
	 * 'Set to' button was clicked 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 * @param	InSetToOption		The type to set to
	 */
	void OnSetToClicked(TWeakPtr< class IStatsViewer > InParentStatsViewer, ESetToOptions::Type InSetToOption) const;

	/** 
	 * Helper function to set the mapping method on selected components 
	 * @param	InParentStatsViewer			The parent stats viewer of this page
	 * @param	bInTextureMapping			Whether to set to texture mode or not
	 * @param	InStaticLightingResolution	The lightmap resolution to set the selected components to
	 */
	void SetMappingMethodOnSelectedComponents(TWeakPtr< class IStatsViewer > InParentStatsViewer, bool bInTextureMapping, int32 InStaticLightingResolution) const;

	/** 
	 * Helper function to swap the mapping method on selected components 
	 * @param	InParentStatsViewer			The parent stats viewer of this page
	 * @param	InStaticLightingResolution	The lightmap resolution to set the selected components to
	 */
	void SwapMappingMethodOnSelectedComponents(TWeakPtr< class IStatsViewer > InParentStatsViewer, int32 InStaticLightingResolution) const;

	/**
	 * Helper function to set static lighting resolution - displays a type-in popup 
	 * that calls back to OnResolutionCommitted when a value is entered
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 * @param	bSwap				Whether we are swapping or setting the value
	 */
	void GetUserSetStaticLightmapResolution(TWeakPtr< class IStatsViewer > InParentStatsViewer, bool bSwap) const;

	/** 
	 * Helper function to set static lighting resolution 
	 * @param	ResolutionText		The text the user typed in
	 * @param	CommitInfo			The type of commit
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 * @param	bSwap				Whether we are swapping or setting the value
	 */
	void OnResolutionCommitted(const FText& ResolutionText, ETextCommit::Type CommitInfo, TWeakPtr< class IStatsViewer > InParentStatsViewer, bool bSwap) const;

	/** 
	 * Delegate to allow is to trigger a refresh on new level 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	void OnEditorNewCurrentLevel( TWeakPtr< class IStatsViewer > InParentStatsViewer );

	/** 
	 * Delegate to allow is to trigger a refresh level selection 
	 * @param	InParentStatsViewer	The parent stats viewer of this page
	 */
	void OnEditorLevelSelected( TWeakPtr< class IStatsViewer > InParentStatsViewer );

private:

	/** Swap combo button */
	TSharedPtr<SComboButton> SwapComboButton;

	/** 'Set to' combo button */
	TSharedPtr<SComboButton> SetToComboButton;

	/** Custom widget for this page */
	TSharedPtr<SWidget> CustomWidget;

	/** Reference to owner of the current popup */
	mutable TWeakPtr<class IMenu> ResolutionEntryMenu;
};

