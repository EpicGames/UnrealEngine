// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsPages/TextureStatsPage.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture.h"
#include "Misc/App.h"
#include "Model.h"
#include "UObject/UObjectIterator.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/Selection.h"
#include "Engine/TextureCube.h"
#include "TextureCompiler.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "ReferencedAssetsUtils.h"
#include "AssetSelection.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "Editor.StatsViewer.TextureStats"

FTextureStatsPage& FTextureStatsPage::Get()
{
	static FTextureStatsPage* Instance = NULL;
	if( Instance == NULL )
	{
		Instance = new FTextureStatsPage;
	}
	return *Instance;
}

/** Helper class to generate statistics */
struct TextureStatsGenerator : public FFindReferencedAssets
{
	TextureStatsGenerator(UWorld* InWorld)
	{
		World = InWorld != nullptr ? InWorld : GWorld;
	}

	UWorld* World;

	/** Textures that should be ignored when taking stats */
	TArray<TWeakObjectPtr<const UTexture>> TexturesToIgnore;

	/** Map so we can track usage per-actor */
	TMap<FString, UTextureStats*> EntryMap;

	void GetObjectsForListMode( const ETextureObjectSets InObjectSet, TArray<UObject*>& OutObjectsToSearch )
	{
		if( InObjectSet == TextureObjectSet_SelectedActors )
		{
			// In this mode only get selected actors
			for( AActor* Actor : FSelectedActorRange(World) )
			{
				OutObjectsToSearch.Add( Actor );
			}
		}
		else if( InObjectSet == TextureObjectSet_SelectedMaterials )
		{
			TArray<FAssetData> SelectedAssets;
			AssetSelectionUtils::GetSelectedAssets( SelectedAssets );

			// In this mode only get selected materials
			for ( auto It = SelectedAssets.CreateConstIterator(); It; ++It )
			{
				if ((*It).IsAssetLoaded())
				{
					UMaterialInterface* Material = Cast<UMaterialInterface>((*It).GetAsset());
					if( Material )
					{
						OutObjectsToSearch.Add( Material );
					}
				}
			}
		}
		else if( InObjectSet == TextureObjectSet_CurrentStreamingLevel )
		{
			// In this mode get all actors in the current level
			for (int32 ActorIdx = 0; ActorIdx < World->GetCurrentLevel()->Actors.Num(); ++ActorIdx )
			{
				OutObjectsToSearch.Add( World->GetCurrentLevel()->Actors[ActorIdx] );
			}
		}
		else if( InObjectSet == TextureObjectSet_AllStreamingLevels )
		{
			// In this mode get all actors in all levels
			for( int32 LevelIdx = 0; LevelIdx < World->GetNumLevels(); ++LevelIdx )
			{
				const ULevel* CurrentLevel = World->GetLevel( LevelIdx );
				for (int32 ActorIdx = 0; ActorIdx < CurrentLevel->Actors.Num(); ++ActorIdx )
				{
					OutObjectsToSearch.Add( CurrentLevel->Actors[ActorIdx] );
				}
			}
		}
	}

	bool IsTextureValidForStats( const UTexture* Texture ) 
	{
		const bool bIsValid = Texture && // texture must exist
			TexturesToIgnore.Find( MakeWeakObjectPtr( Texture ) ) == INDEX_NONE && // texture is not one that should be ignored
			( Texture->IsA( UTexture2D::StaticClass() ) || Texture->IsA( UTextureCube::StaticClass() ) ); // texture is valid texture class for stat purposes

#if 0 // @todo TextureInfoInUE UTextureCube::GetFace doesn't exist
		UTextureCube* CubeTex = Cast<UTextureCube>( Texture );
		if( CubeTex )
		{
			// If the passed in texture is a cube, add all faces of the cube to the ignore list since the cube will account for those
			for( int32 FaceIdx = 0; FaceIdx < 6; ++FaceIdx )
			{
				TexturesToIgnore.Add( MakeWeakObjectPtr( CubeTex->GetFace( FaceIdx ) ) );
			}
		}
#endif

		return bIsValid;
	}

	void BuildReferencingData( ETextureObjectSets InObjectSet )
	{
		// Don't check for BSP mats if the list mode needs something to be selected
		if( InObjectSet != TextureObjectSet_SelectedActors && InObjectSet != TextureObjectSet_SelectedMaterials )
		{
			TSet<UObject*> BspMats;
			// materials to a temp list
			for (int32 Index = 0; Index < World->GetModel()->Surfs.Num(); Index++)
			{
				// No point showing the default material
				if (World->GetModel()->Surfs[Index].Material != NULL)
				{
					BspMats.Add(World->GetModel()->Surfs[Index].Material);
				}
			}
			// If any BSP surfaces are selected
			if (BspMats.Num() > 0)
			{
				FReferencedAssets* Referencer = new(Referencers) FReferencedAssets(World->GetModel());

				// Now copy the array
				Referencer->AssetList = BspMats;
				ReferenceGraph.Add(World->GetModel(), ObjectPtrWrap(BspMats));
			}
		}

		if (World->GetOutermost() == GetTransientPackage())
		{
			// Do not ignore the transient package as our world lives there.
			IgnorePackages.Remove(GetTransientPackage());
		}

		// this is the maximum depth to use when searching for references
		const int32 MaxRecursionDepth = 0;

		// Mark all objects so we don't get into an endless recursion
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			// Skip the level, world, and any packages that should be ignored
			if ( ShouldSearchForAssets(*It,ObjectPtrDecay(IgnoreClasses), ObjectPtrDecay(IgnorePackages),false) )
			{
				It->Mark(OBJECTMARK_TagExp);
			}
			else
			{
				It->UnMark(OBJECTMARK_TagExp);
			}
		}

		// Get the objects to search for texture references
		TArray< UObject* > ObjectsToSearch;
		GetObjectsForListMode( InObjectSet, ObjectsToSearch );

		TArray<UObject*> ObjectsToSkip;

		for( int32 ObjIdx = 0; ObjIdx < ObjectsToSearch.Num(); ++ObjIdx )
		{
			UObject* CurrentObject = ObjectsToSearch[ ObjIdx ];
			if ( !ObjectsToSkip.Contains(CurrentObject) )
			{
				// Create a new entry for this actor
				FReferencedAssets* Referencer = new(Referencers) FReferencedAssets(CurrentObject);

				// Add to the list of referenced assets
				FFindAssetsArchive(CurrentObject,Referencer->AssetList,&ReferenceGraph,MaxRecursionDepth,false,false);
			}
		}
	}

	FString GetTexturePath(const FString &FullyQualifiedPath)
	{
		const int32 Index = FullyQualifiedPath.Find(TEXT(".") );

		if(Index == INDEX_NONE)
		{
			return TEXT("");
		}
		else
		{
			return FullyQualifiedPath.Left(Index);
		}
	}

	void AddEntry( const UTexture* InTexture, const AActor* InActorUsingTexture, TArray< TWeakObjectPtr<UObject> >& OutObjects )
	{
		UTextureStats* Entry = NULL;
		UTextureStats** EntryPtr = EntryMap.Find(InTexture->GetPathName());
		if(EntryPtr == NULL)
		{
			Entry = NewObject<UTextureStats>();
			Entry->AddToRoot();
			OutObjects.Add(Entry);
			EntryMap.Add(InTexture->GetPathName(), Entry);

			Entry->Texture = MakeWeakObjectPtr(const_cast<UTexture*>(InTexture));
			Entry->Path = GetTexturePath(InTexture->GetPathName());
			Entry->Group = (TextureGroup)InTexture->LODGroup;

			// Avoid pulling on the platform data while compilation is pending
			// as it would stall until the compilation is finished. We will report
			// pending compiling instead for improved UI responsiveness during async
			// compilation.
			const bool bIsCompiling = InTexture->IsCompiling();

			if (bIsCompiling)
			{
				// Make it abondantly clear that the value is wrong until the compilation is finished.
				Entry->CurrentKB = -1.0f; 
				Entry->FullyLoadedKB = -1.0f;
			}
			else
			{
				Entry->CurrentKB = InTexture->CalcTextureMemorySizeEnum( TMC_ResidentMips ) / 1024.0f;
				Entry->FullyLoadedKB = InTexture->CalcTextureMemorySizeEnum( TMC_AllMipsBiased ) / 1024.0f;
			}

			Entry->CombinedLODBias = InTexture->GetCachedLODBias();
			Entry->TextureLODBias = InTexture->LODBias;

			const FTexture* Resource = InTexture->GetResource();
			if(Resource)
			{
				Entry->LastTimeRendered = (float)FMath::Max( FApp::GetLastTime() - Resource->LastRenderTime, 0.0 );
			}

			Entry->Virtual = bIsCompiling ? TEXT("Unknown") : (InTexture->IsCurrentlyVirtualTextured() ? TEXT("YES") : TEXT("NO"));

			const UTexture2D* Texture2D = Cast<const UTexture2D>(InTexture);
			if( Texture2D )
			{
				Entry->Format = Texture2D->GetPixelFormat();
				Entry->Type = TEXT("2D"); 

				// Calculate in game current dimensions 
				const int32 DroppedMips = Texture2D->GetNumMips() - Texture2D->GetNumResidentMips();
				Entry->CurrentDim.X = Texture2D->GetSizeX() >> DroppedMips;
				Entry->CurrentDim.Y = Texture2D->GetSizeY() >> DroppedMips;

				// Calculate the max dimensions
				Entry->MaxDim.X = Texture2D->GetSizeX() >> Entry->CombinedLODBias;
				Entry->MaxDim.Y = Texture2D->GetSizeY() >> Entry->CombinedLODBias;
			}
			else
			{
				// Check if the texture is a TextureCube
				const UTextureCube* TextureCube = Cast<const UTextureCube>(InTexture);
				if(TextureCube)
				{
					Entry->Format = TextureCube->GetPixelFormat();
					Entry->Type = TEXT("Cube"); 

#if 0 // @todo TextureInfoInUE UTextureCube::GetFace doesn't exist.
					// Calculate in game current dimensions 
					// Use one face of the texture cube to calculate in game size
					UTexture2D* Face = TextureCube->GetFace(0);
					const int32 DroppedMips = Face->GetNumMips() - Face->ResidentMips;
					Entry->CurrentDim.X = Face->GetSizeX() >> DroppedMips;
					Entry->CurrentDim.Y = Face->GetSizeY() >> DroppedMips;

					// Calculate the max dimensions
					Entry->MaxDim.X = Face->GetSizeX() >> Entry->CombinedLODBias;
					Entry->MaxDim.Y = Face->GetSizeY() >> Entry->CombinedLODBias;
#else
					// Calculate in game current dimensions 
					Entry->CurrentDim.X = TextureCube->GetSizeX() >> Entry->CombinedLODBias;
					Entry->CurrentDim.Y = TextureCube->GetSizeY() >> Entry->CombinedLODBias;

					// Calculate the max dimensions
					Entry->MaxDim.X = TextureCube->GetSizeX() >> Entry->CombinedLODBias;
					Entry->MaxDim.Y = TextureCube->GetSizeY() >> Entry->CombinedLODBias;
#endif
				}
			}

			if (bIsCompiling)
			{
				// Replace the texture type by something that convey meaning to the user that compilation is pending.
				Entry->Type = TEXT("Compiling...");
			}
		}
		else
		{
			Entry = *EntryPtr;
		}

		if( InActorUsingTexture != NULL && !Entry->Actors.Contains(InActorUsingTexture) )
		{
			Entry->Actors.Add(MakeWeakObjectPtr(const_cast<AActor*>(InActorUsingTexture)));
			Entry->NumUses++;
		}
	}

	void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects )
	{
		for (int32 RefIndex = 0; RefIndex < Referencers.Num(); RefIndex++)
		{
			const TSet<UObject*> &AssetList = Referencers[RefIndex].AssetList;

			// Look at each referenced asset
			for(TSet<UObject*>::TConstIterator SetIt(AssetList); SetIt; ++SetIt)
			{
				const UObject* Asset = *SetIt;

				const UTexture* CurrentTexture = Cast<const UTexture>(Asset);

				if(IsTextureValidForStats(CurrentTexture))
				{
					AActor* ActorUsingTexture = Cast<AActor>(Referencers[RefIndex].Referencer);

					// referenced by an actor
					AddEntry(CurrentTexture, ActorUsingTexture, OutObjects);
				}

				const UPrimitiveComponent* ReferencedComponent = Cast<const UPrimitiveComponent>(Asset);
				if (ReferencedComponent)
				{
					// If the referenced asset is a primitive component get the materials used by the component
					TArray<UMaterialInterface*> UsedMaterials;
					ReferencedComponent->GetUsedMaterials( UsedMaterials );
					for(int32 MaterialIndex = 0; MaterialIndex < UsedMaterials.Num(); MaterialIndex++)
					{
						// For each material, find the textures used by that material and add it to the stat list
						const UMaterialInterface* CurrentMaterial = UsedMaterials[MaterialIndex];
						if(CurrentMaterial)
						{
							TArray<UTexture*> UsedTextures;

							CurrentMaterial->GetUsedTextures(UsedTextures, GetCurrentMaterialQualityLevelChecked());
							for(int32 TextureIndex = 0; TextureIndex < UsedTextures.Num(); TextureIndex++)
							{
								UTexture* CurrentUsedTexture = UsedTextures[TextureIndex];

								if(IsTextureValidForStats(CurrentUsedTexture))
								{
									AActor* ActorUsingTexture = Cast<AActor>(Referencers[RefIndex].Referencer);

									// referenced by an material
									AddEntry(CurrentUsedTexture, ActorUsingTexture, OutObjects);
								}
							}
						}
					}
				}
			}
		}
	}
};

void FTextureStatsPage::Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const
{
	TextureStatsGenerator Generator( GetWorld() );

	Generator.BuildReferencingData( (ETextureObjectSets)ObjectSetIndex );
	Generator.Generate( OutObjects );
}

void FTextureStatsPage::GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const
{
	if(InObjects.Num())
	{
		UTextureStats* TotalEntry = NewObject<UTextureStats>();

		for( auto It = InObjects.CreateConstIterator(); It; ++It )
		{
			UTextureStats* StatsEntry = Cast<UTextureStats>( It->Get() );
			if (StatsEntry->CurrentKB >= 0)
			{
				TotalEntry->CurrentKB += StatsEntry->CurrentKB;
			}
			
			if (StatsEntry->FullyLoadedKB >= 0)
			{
				TotalEntry->FullyLoadedKB += StatsEntry->FullyLoadedKB;
			}

			TotalEntry->NumUses += StatsEntry->NumUses;
		}

		OutTotals.Add( TEXT("CurrentKB"), FText::AsNumber( TotalEntry->CurrentKB ) );
		OutTotals.Add( TEXT("FullyLoadedKB"), FText::AsNumber( TotalEntry->FullyLoadedKB ) );
		OutTotals.Add( TEXT("NumUses"), FText::AsNumber( TotalEntry->NumUses ) );
	}
}

void FTextureStatsPage::OnEditorSelectionChanged( UObject* NewSelection, TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == TextureObjectSet_SelectedActors || ObjSetIndex == TextureObjectSet_SelectedMaterials )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FTextureStatsPage::OnEditorNewCurrentLevel( TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	if(InParentStatsViewer.IsValid())
	{
		const int32 ObjSetIndex = InParentStatsViewer.Pin()->GetObjectSetIndex();
		if( ObjSetIndex == TextureObjectSet_CurrentStreamingLevel )
		{
			InParentStatsViewer.Pin()->Refresh();
		}
	}
}

void FTextureStatsPage::OnAssetPostCompile( const TArray<FAssetCompileData>& CompiledAssets, TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	// Only trigger a refresh on the last compiled texture because rebuilding all the stats is costly enough 
	// that we don't want to trigger a rebuild for every single texture that finishes compiling.
	if (FTextureCompilingManager::Get().GetNumRemainingTextures() == 0)
	{
		if (InParentStatsViewer.IsValid())
		{
			// Make sure that the event is not sent for some other asset type since we don't want to
			// trigger refresh unless it concerns textures.
			const bool bContainsTextures =
				CompiledAssets.ContainsByPredicate(
					[](const FAssetCompileData& AssetCompileData)
					{
						return Cast<UTexture>(AssetCompileData.Asset.Get()) != nullptr;
					});

			if (bContainsTextures)
			{
				InParentStatsViewer.Pin()->Refresh();
			}
		}
	}
}

void FTextureStatsPage::OnShow( TWeakPtr< IStatsViewer > InParentStatsViewer )
{
	// register delegates for scene changes we are interested in
	USelection::SelectionChangedEvent.AddRaw(this, &FTextureStatsPage::OnEditorSelectionChanged, InParentStatsViewer);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FTextureStatsPage::OnEditorNewCurrentLevel, InParentStatsViewer);
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddRaw(this, &FTextureStatsPage::OnAssetPostCompile, InParentStatsViewer);
}

void FTextureStatsPage::OnHide()
{
	// unregister delegates
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);
	USelection::SelectionChangedEvent.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE
