// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStatsViewer.h"
#include "StatsPage.h"
#include "TextureStats.h"
#include "AssetCompilingManager.h"

/** Stats page representing texture stats */
class FTextureStatsPage : public FStatsPage<UTextureStats>
{
public:
	/** Singleton accessor */
	static FTextureStatsPage& Get();

	/** Begin IStatsPage interface */
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const override;
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const override;
	virtual void OnShow( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override;
	virtual void OnHide() override;
	/** End IStatsPage interface */

	virtual ~FTextureStatsPage() = default;

private:

	/** Delegate to trigger a refresh on actor selection */
	void OnEditorSelectionChanged( UObject* NewSelection, TWeakPtr< class IStatsViewer > InParentStatsViewer );

	/** Delegate to trigger a refresh on new level */
	void OnEditorNewCurrentLevel( TWeakPtr< class IStatsViewer > InParentStatsViewer );

	/** Delegate to trigger a refresh when a texture finishes compiling */
	void OnAssetPostCompile( const TArray<FAssetCompileData>& CompiledAssets, TWeakPtr< IStatsViewer > InParentStatsViewer );
};

