// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsViewerModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "SStatsViewer.h"
#include "StatsPageManager.h"
#include "StatsPages/CookerStatsPage.h"
#include "StatsPages/LightingBuildInfoStatsPage.h"
#include "StatsPages/PrimitiveStatsPage.h"
#include "StatsPages/StaticMeshLightingInfoStatsPage.h"
#include "StatsPages/TextureStatsPage.h"
#include "StatsPages/ShaderCookerStatsPage.h"
#include "ObjectHyperlinkColumn.h"


#define LOCTEXT_NAMESPACE "Editor.StatsViewer"

IMPLEMENT_MODULE( FStatsViewerModule, StatsViewer );

const FName StatsViewerApp = FName("StatsViewerApp");

static const FName CookerStatsPage = FName("CookerStats");
static const FName LightingBuildInfoPage = FName("LightingBuildInfo");
static const FName PrimitiveStatsPage = FName("PrimitiveStats");
static const FName StaticMeshLightingInfoPage = FName("StaticMeshLightingInfo");
static const FName TextureStatsPage = FName("TextureStats");
static const FName ShaderCookerStatsPage = FName("ShaderCookerStats");

namespace StatsViewerModuleUtils
{
	// Allocate new page and add it to the specified page manager.
	template <typename T>
	void RegisterPageInstance(UWorld& InWorld, int32 PageType, TSharedRef<FStatsPageManager> StatsPageManager)
	{
		TSharedPtr< T > PageInstance = MakeShared< T >();
		PageInstance->SetWorld(InWorld);
		StatsPageManager->RegisterPage(PageType, PageInstance.ToSharedRef());
	}
}

void FStatsViewerModule::StartupModule()
{
	FStatsPageManager::Get().RegisterPage( EStatsPage::CookerStats, MakeShareable(&FCookerStatsPage::Get()) );
	FStatsPageManager::Get().RegisterPage( EStatsPage::LightingBuildInfo, MakeShareable(&FLightingBuildInfoStatsPage::Get()) );
	FStatsPageManager::Get().RegisterPage( EStatsPage::PrimitiveStats, MakeShareable(&FPrimitiveStatsPage::Get()) );
	FStatsPageManager::Get().RegisterPage( EStatsPage::StaticMeshLightingInfo, MakeShareable(&FStaticMeshLightingInfoStatsPage::Get()) );
	FStatsPageManager::Get().RegisterPage( EStatsPage::TextureStats, MakeShareable(&FTextureStatsPage::Get()) );
	FStatsPageManager::Get().RegisterPage( EStatsPage::ShaderCookerStats, MakeShareable(&FShaderCookerStatsPage::Get()) );
}

void FStatsViewerModule::ShutdownModule()
{
	FStatsPageManager::Get().UnregisterAllPages();
}

TSharedRef< IStatsViewer > FStatsViewerModule::CreateStatsViewer() const
{
	return SNew( SStatsViewer )
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() );
}

TSharedRef< IStatsViewer > FStatsViewerModule::CreateStatsViewer(UWorld& InWorld, uint32 EnabledDefaultPagesMask, const FName& ViewerName) const
{
	TSharedPtr< FStatsPageManager > StatsPageManager = MakeShared< FStatsPageManager >(ViewerName);

	if ((1 << EStatsPage::CookerStats) & EnabledDefaultPagesMask)
	{
		StatsViewerModuleUtils::RegisterPageInstance< FCookerStatsPage >(InWorld, EStatsPage::CookerStats, StatsPageManager.ToSharedRef());
	}
	if ((1 << EStatsPage::LightingBuildInfo) & EnabledDefaultPagesMask)
	{
		StatsViewerModuleUtils::RegisterPageInstance< FLightingBuildInfoStatsPage >(InWorld, EStatsPage::LightingBuildInfo, StatsPageManager.ToSharedRef());
	}
	if ((1 << EStatsPage::PrimitiveStats) & EnabledDefaultPagesMask)
	{
		StatsViewerModuleUtils::RegisterPageInstance< FPrimitiveStatsPage >(InWorld, EStatsPage::PrimitiveStats, StatsPageManager.ToSharedRef());
	}
	if ((1 << EStatsPage::StaticMeshLightingInfo) & EnabledDefaultPagesMask)
	{
		StatsViewerModuleUtils::RegisterPageInstance< FStaticMeshLightingInfoStatsPage >(InWorld, EStatsPage::StaticMeshLightingInfo, StatsPageManager.ToSharedRef());
	}
	if ((1 << EStatsPage::TextureStats) & EnabledDefaultPagesMask)
	{
		StatsViewerModuleUtils::RegisterPageInstance< FTextureStatsPage >(InWorld, EStatsPage::TextureStats, StatsPageManager.ToSharedRef());
	}

	return SNew(SStatsViewer)
		.StatsPageManager(StatsPageManager)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
}

TSharedRef< class IPropertyTableCustomColumn > FStatsViewerModule::CreateObjectCustomColumn(const FObjectHyperlinkColumnInitializationOptions& InOptions) const
{
	return MakeShareable(new FObjectHyperlinkColumn(InOptions));
}

void FStatsViewerModule::RegisterPage( TSharedRef< IStatsPage > InPage )
{
	FStatsPageManager::Get().RegisterPage( InPage);
}

void FStatsViewerModule::UnregisterPage( TSharedRef< IStatsPage > InPage )
{
	FStatsPageManager::Get().UnregisterPage(InPage);
}

TSharedPtr< IStatsPage > FStatsViewerModule::GetPage( EStatsPage::Type InType )
{
	switch( InType )
	{
	case EStatsPage::CookerStats:
		return GetPage(CookerStatsPage);
	case EStatsPage::LightingBuildInfo:
		return GetPage(LightingBuildInfoPage);
	case EStatsPage::PrimitiveStats:
		return GetPage(PrimitiveStatsPage);
	case EStatsPage::StaticMeshLightingInfo:
		return GetPage(StaticMeshLightingInfoPage);
	case EStatsPage::TextureStats:
		return GetPage(TextureStatsPage);
	case EStatsPage::ShaderCookerStats:
		return GetPage(ShaderCookerStatsPage);
	default:
		return NULL;
	}
}

TSharedPtr< IStatsPage > FStatsViewerModule::GetPage( const FName& InPageName )
{
	return FStatsPageManager::Get().GetPage(InPageName);
}

void FStatsViewerModule::Clear()
{
	for( int32 PageIndex = 0; PageIndex < FStatsPageManager::Get().NumPages(); PageIndex++ )
	{
		FStatsPageManager::Get().GetPageByIndex(PageIndex)->Clear();
	}
}

#undef LOCTEXT_NAMESPACE
