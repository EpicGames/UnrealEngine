// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsViewerUtils.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Texture.h"

namespace StatsViewerUtils
{

	AActor* GetActor( const TWeakObjectPtr<UObject>& InObject )
	{
		AActor* Actor = NULL;
		if(InObject.IsValid())
		{
			// Is this an actor, or an object held by an actor?
			Actor = Cast<AActor>(InObject.Get());
			if(Actor == NULL)
			{
				UActorComponent* Component = Cast<UActorComponent>(InObject.Get());
				if(Component != NULL)
				{
					Actor = Cast<AActor>(Component->GetOuter());
				}
			}
		}

		return Actor;
	}

	FText GetAssetName( const TWeakObjectPtr<UObject>& InObject )
	{
		FString Name = TEXT("");

		// Is this an object held by an actor?
		AActor* Actor = GetActor( InObject );
		if (Actor)
		{
			Name = Actor->GetName();
		}
		else 
		{
			// or is the object a texture?
			UTexture* Texture = Cast<UTexture>(InObject.Get());
			if( Texture )
			{
				const FString FullyQualifiedPath = Texture->GetPathName();
				const int32 Index = Texture->GetPathName().Find(TEXT("."));

				if(Index == INDEX_NONE)
				{
					Name = FullyQualifiedPath;
				}
				else
				{
					Name = FullyQualifiedPath.Right(FullyQualifiedPath.Len() - Index - 1);
				}
			}
			else
			{
				Name = InObject->GetName();
			}
		}

		return FText::FromString( Name );
	}

}
