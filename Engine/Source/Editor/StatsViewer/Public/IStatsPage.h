// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStatsViewer.h"

/**
 * The public interface for a stats page.
 * A stats page displays a collection of identically-typed objects with columns based around their UPROPERTY()'s
 * The table does its best to display the relevant information.
 *
 * Objects UCLASS metadata required are:
 * DisplayName		- The name displayed in the page selection dropdown.
 * Tooltip			- The tooltip displayed in the page selection dropdown.
 * ObjectSetType	- A text representation of the UENUM used for object set enumeration.
 *
 * The object set UENUM also uses metadata to display certain information. UENUM metadata syntax differs from UCLASS metadata
 * as it is specified via the UMETA() tag, rather than meta=() UPROPERTY subtags.
 * DisplayName		- The name displayed in the object set dropdown.
 * ToolTip			- The tooltip displayed over the object set dropdown.
 *
 * The UI supports the following UPROPERTY metadata:
 * ColumnWidth		- Integer value. The (proportionally-based, not absolute) width of the properties column.
 * ShowTotal		- Either true or false. Whether the column header should attempt to show a total (provided in Generate() via the OutTotals map).
 * SortMode			- Either Ascending or Descending. If this is specified then the properties column will be sorted on table creation.
 * Unit				- Text value displayed next to table entries and totals
 */
class IStatsPage
{
public:

	/** 
	 * Clears any entries added via AddEntry() 
	 * Not all pages have to override this - only ones that accept transient data output
	 * from processes that do not persist in the editor
	 */
	virtual void Clear() {}

	/** 
	 * Adds a stats entry to the page
	 * Not all pages have to override this - only ones that accept transient data output
	 * from processes that do not persist in the editor
	 */
	virtual void AddEntry( UObject* InEntry ) {}

	/** 
	 * Tries to switch the currently displayed page to this one 
	 * @param	bShow	Whether to show the page (passing false will not hide the page)
	 */
	virtual void Show( bool bShow = true ) = 0;

	/** Check if this page wants to show itself */
	virtual bool IsShowPending() const = 0;

	/** 
	 * Sends a requests to the stats page to refresh itself the next chance it gets 
	 * @param	bRefresh	Whether to refresh the page (the page will refresh on the stats viewers next tick)
	 */
	virtual void Refresh( bool bRefresh = true ) = 0;

	/** Check if this page wants to refresh itself */
	virtual bool IsRefreshPending() const = 0;

	/** Get the name of the entry type */
	virtual FName GetName() const = 0;

	/** Get the name of the entry type to be displayed in the page selection dropdown */
	virtual const FText GetDisplayName() const = 0;

	/** Get the tooltip to be displayed over the page selection dropdown */
	virtual const FText GetToolTip() const = 0;

	/** Get the number of object sets this page supports */
	virtual int32 GetObjectSetCount() const = 0;

	/** 
	 * Get the name of the object set, to be displayed in the dropdown 
	 * @param	InObjectSetIndex	The index of the object set to get the name of
	 */
	virtual FString GetObjectSetName( int32 InObjectSetIndex ) const = 0;

	/** 
	 * Get the tooltip of the object set, to be displayed over the object set the dropdown 
	 * @param	InObjectSetIndex	The index of the object set to get the tooltip for
	 */
	virtual FString GetObjectSetToolTip( int32 InObjectSetIndex ) const = 0;

	/** 
	 * Get the class of the entry we handle 
	 * This is needed to display the search filter's combo button
	 */
	virtual UClass* GetEntryClass() const = 0;

	/** 
	 * Fill the output array with statistic objects to be displayed
	 *
	 * @param OutObjects	The object array to be filled
	 */
	virtual void Generate( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const = 0;

	/** 
	 * Totals are displayed by mapping the CPP name of the columns property to 
	 * the total string in the OutTotals map.
	 *
	 * @param OutTotals		A map of Property->GetNameCPP() -> Total string
	 */
	virtual void GenerateTotals( const TArray< TWeakObjectPtr<UObject> >& InObjects, TMap<FString, FText>& OutTotals ) const = 0;

	/** 
	 * Get custom filter to display in the top part of the stats viewer, can return nullptr 
	 * @param	InParentStatsViewer	The parent stats viewer
	 */
	virtual TSharedPtr<SWidget> GetCustomFilter( TWeakPtr< class IStatsViewer > InParentStatsViewer ) = 0;

	/** 
	 * Get custom widget to display in the top part of the stats viewer, can return nullptr 
	 * @param	InParentStatsViewer	The parent stats viewer
	 */
	virtual TSharedPtr<SWidget> GetCustomWidget( TWeakPtr< class IStatsViewer > InParentStatsViewer ) = 0;

	/** 
	 * Called back each time the page is shown 
	 * @param	InParentStatsViewer	The parent stats viewer
	 */
	virtual void OnShow( TWeakPtr< class IStatsViewer > InParentStatsViewer ) {}

	/** Called back each time the page is hidden */
	virtual void OnHide() {}

	/** 
	 * Set the currently displayed object set 
	 * @param	InObjectSetIndex	The object set index to set
	 */
	virtual void SetSelectedObjectSet( int32 InObjectSetIndex ) = 0;

	/** Get the currently displayed object set */
	virtual int32 GetSelectedObjectSet() const = 0;

	/**
	 * Get any column customizations that this page wants to use.
	 * @param	OutCustomColumns	The array to be filled in containing column customizations.
	 */
	virtual void GetCustomColumns(TArray< TSharedRef< class IPropertyTableCustomColumn > >& OutCustomColumns) const = 0;

	/**
	 * Set a specific world to use for page statistics.
	 * @param	InWorld The world to use.
	 */
	virtual void SetWorld(UWorld& InWorld) = 0;

	/** Get the world used for page statistics. */
	virtual UWorld* GetWorld() const = 0;
};
