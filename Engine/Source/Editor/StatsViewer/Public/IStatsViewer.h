// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "IPropertyTable.h"

/**
 * The public interface for the Stats Viewer widget
 */
class IStatsViewer : public SCompoundWidget
{

public:

	/** Sends a requests to the Stats Viewer to refresh itself the next chance it gets */
	virtual void Refresh() = 0;

	/** Get the property table we are displaying */
	virtual TSharedPtr< class IPropertyTable > GetPropertyTable() = 0;

	/** Get the currently selected object set */
	virtual int32 GetObjectSetIndex() const = 0;

	virtual void SwitchAndFilterPage(int32 Table, const FString& FilterValue, const FString& FilterProperty) = 0;
};

