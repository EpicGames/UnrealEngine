// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Delegate called to generate a custom widget for display in a cell of a UObject custom column */
DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<class SWidget>, FOnGenerateWidget, const FText& /** InValueAsText */, const TWeakObjectPtr< UObject >& /** InObjectPtr */);

/** Delegate called when a hyperlink is clicked in a UObject custom column */
DECLARE_DELEGATE_OneParam(FOnObjectHyperlinkClicked, const TWeakObjectPtr< UObject >& /** InObjectPtr */);

/** Delegate used to query whether a UClass is supported by a UObject custom column */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsClassSupported, const UClass* /** InClassToQuery */);

/**
 * Struct used to further customize object hyperlink custom columns.
 */
struct FObjectHyperlinkColumnInitializationOptions
{
	/** 
	 * Delegate called to generate a custom widget for display in a cell of a UObject custom column.
	 * Note that overriding this means that the calling code is responsible for handling interactions
	 * with the widget, i.e. OnObjectHyperlinkClicked will not be called.
	 */
	FOnGenerateWidget OnGenerateWidget;

	/** 
	 * Delegate called when a hyperlink is clicked in a UObject custom column 
	 * Note that this is not called if OnGenerateWidget is bound.
	 */
	FOnObjectHyperlinkClicked OnObjectHyperlinkClicked;

	/**
	 * Delegate used to query whether a UClass is supported by a UObject custom column. 
	 * Use this to implement custom columns that support your weak object type or
	 * override an existing internal implementation.
	 */
	FOnIsClassSupported OnIsClassSupported;
};
