// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SNullWidget.h"
#include "IPropertyTableCellPresenter.h"

/** 
 * Boilerplate implementations of IPropertyTableCellPresenter functions.
 * Here so we dont have to override all these functions each time.
 */
class FStatsCellPresenter : public IPropertyTableCellPresenter
{
public:
	/** Begin IPropertyTableCellPresenter interface */
	virtual bool RequiresDropDown() override
	{
		return false;
	}

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override
	{
		return ConstructDisplayWidget();
	}

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override
	{
		return SNullWidget::NullWidget;
	}

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override
	{
		return SNullWidget::NullWidget;
	}

	virtual bool HasReadOnlyEditMode() override 
	{ 
		return true; 
	}

	virtual FString GetValueAsString() override
	{
		return Text.ToString();
	}

	virtual FText GetValueAsText() override
	{
		return Text;
	}
	/** End IPropertyTableCellPresenter interface */

protected:

	/** 
	 * The text we display - we just store this as we never need to *edit* properties in the stats viewer,
	 * only view them. Therefore we dont need to dynamically update this string.
	 */
	FText Text;
};
