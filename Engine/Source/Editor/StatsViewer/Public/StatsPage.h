// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "UObject/Class.h"
#include "IStatsViewer.h"
#include "IStatsPage.h"

/** 
 * Template for all stats pages/factories. 
 * These classes generate uniform arrays of identically-typed objects that
 * are displayed in a PropertyTable.
 * Boilerplate implementations are below that all pages in this module currently more-or-less share
 */
template <typename Entry>
class FStatsPage : public IStatsPage
{
public:
	FStatsPage()
	{
		FString EnumName = Entry::StaticClass()->GetName();
		EnumName += TEXT(".");
		EnumName += Entry::StaticClass()->GetMetaData( TEXT("ObjectSetType") );
		ObjectSetEnum = FindObject<UEnum>( nullptr, *EnumName );
		bRefresh = false;
		bShow = false;
		ObjectSetIndex = 0;
	}

	virtual ~FStatsPage() = default;

	/** Begin IStatsPage interface */
	virtual void Show( bool bInShow = true ) override
	{
		bShow = bInShow;
	}

	virtual bool IsShowPending() const override
	{
		return bShow;
	}

	virtual void Refresh( bool bInRefresh = true ) override
	{
		bRefresh = bInRefresh;
	}

	virtual bool IsRefreshPending() const override
	{
		return bRefresh;
	}

	virtual FName GetName() const override
	{
		return Entry::StaticClass()->GetFName();
	}

	virtual const FText GetDisplayName() const override
	{
		return Entry::StaticClass()->GetDisplayNameText();
	}

	virtual const FText GetToolTip() const override
	{
		return Entry::StaticClass()->GetToolTipText();
	}

	virtual int32 GetObjectSetCount() const override
	{
		if(ObjectSetEnum != nullptr)
		{
			return ObjectSetEnum->NumEnums() - 1;
		}
		return 1;
	}

	virtual FString GetObjectSetName( int32 InObjectSetIndex ) const override
	{
		if(ObjectSetEnum != nullptr)
		{
			return ObjectSetEnum->GetDisplayNameTextByIndex( InObjectSetIndex ).ToString();
		}

		// if no valid object set, return a static empty string
		static FString EmptyString;
		return EmptyString;
	}

	virtual FString GetObjectSetToolTip( int32 InObjectSetIndex ) const override
	{
		if(ObjectSetEnum != nullptr)
		{
			return ObjectSetEnum->GetToolTipTextByIndex( InObjectSetIndex ).ToString();
		}

		// if no valid object set, return a static empty string
		static FString EmptyString;
		return EmptyString;
	}

	virtual UClass* GetEntryClass() const override
	{
		return Entry::StaticClass();
	}

	virtual TSharedPtr<SWidget> GetCustomFilter( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override
	{
		return nullptr;
	}

	virtual TSharedPtr<SWidget> GetCustomWidget( TWeakPtr< class IStatsViewer > InParentStatsViewer ) override
	{
		return nullptr;
	}

	virtual void SetSelectedObjectSet( int32 InObjectSetIndex ) override
	{
		ObjectSetIndex = InObjectSetIndex;
	}

	virtual int32 GetSelectedObjectSet() const override
	{
		return ObjectSetIndex;
	}

	virtual void GetCustomColumns(TArray< TSharedRef< class IPropertyTableCustomColumn > >& OutCustomColumns) const override
	{
	}

	virtual void SetWorld( UWorld& InWorld ) override
	{
		StatsWorld = MakeWeakObjectPtr<UWorld>( &InWorld );
	}

	virtual UWorld* GetWorld() const override;
	/** End IStatsPage interface */

protected:

	/** The enum we use for our object set */
	UEnum* ObjectSetEnum;
	
	/** Selected object set index */
	int32 ObjectSetIndex;

	/** Flag to refresh the page */
	bool bRefresh;

	/** Flag to show the page */
	bool bShow;	

	/**
	 * The world to use for pages that need to query actors/components.
	 *  Can be null, in which case GWorld will be used.
	 */
	TWeakObjectPtr< UWorld > StatsWorld;
};

template< typename Entry >
inline UWorld* FStatsPage< Entry >::GetWorld() const
{
	UWorld* World = GWorld;

	if ( StatsWorld.IsValid() )
	{
		World = StatsWorld.Get();
	}
	else if (GEditor && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld))
	{
		if (GEditor->PlayWorld)
		{
			World = GEditor->PlayWorld;
		}
		else
		{
			FWorldContext* WorldContext = GEngine->GetWorldContextFromPIEInstance(0);
			UWorld* SIEWorld = WorldContext ? WorldContext->World() : nullptr;
			if (SIEWorld)
			{
				World = SIEWorld;
			}
		}
	}

	return World;
}
