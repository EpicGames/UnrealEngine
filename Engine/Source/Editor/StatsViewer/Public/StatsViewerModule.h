// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "IStatsViewer.h"
#include "IStatsPage.h"

extern const FName StatsViewerApp;

/** The predefined stats pages built into this module */
namespace EStatsPage
{
	enum Type
	{
		CookerStats,
		LightingBuildInfo,
		PrimitiveStats,
		StaticMeshLightingInfo,
		TextureStats,
		ShaderCookerStats,
	};
}

class FStatsViewerModule : public IModuleInterface
{
public:
	/** Begin IModuleInterface interface */
	virtual void StartupModule();
	virtual void ShutdownModule();
	/** End IModuleInterface interface */

	/**
	 * Creates a stats viewer widget
	 *
	 * @return	New stats viewer widget
	 */
	virtual TSharedRef< IStatsViewer > CreateStatsViewer() const;

	/**
	 * Creates a customized stats viewer widget.
	 *
	 * @param InWorld					Use this world (instead of default one) for all statistics
	 * @param EnabledDefaultPagesMask	The default pages that will be available for this stats viewer
	 * @param ViewerName				A unique name for this stats viewer (used as a configuration settings key)
	 *
	 * @return	New stats viewer widget
	 */
	virtual TSharedRef< IStatsViewer > CreateStatsViewer( UWorld& InWorld, uint32 EnabledDefaultPagesMask, const FName& ViewerName ) const;

	/**
	 * Creates a stats viewer custom column, supporting weak object references.
	 * @param	InOptions	Options used to configure the custom column.
	 * @return	New column customization
	 */
	virtual TSharedRef< class IPropertyTableCustomColumn > CreateObjectCustomColumn(const struct FObjectHyperlinkColumnInitializationOptions& InOptions) const;

	/** 
	 * Register a page for this module to use
	 * @param	InPage	The page to register
	 */
	virtual void RegisterPage( TSharedRef< IStatsPage > InPage );

	/** 
	 * Unregister a page for this module to use
	 * @param	InPage	The page to unregister
	 */
	virtual void UnregisterPage( TSharedRef< IStatsPage > InPage );

	/** 
	 * Get a page of the stats module by enum type
	 */
	virtual TSharedPtr< IStatsPage > GetPage( EStatsPage::Type InType );

	/** 
	 * Get a page of the stats module by name
	 */
	virtual TSharedPtr< IStatsPage > GetPage( const FName& InPageName );

	/** 
	 * Clears all pages
	 */
	virtual void Clear();
};
