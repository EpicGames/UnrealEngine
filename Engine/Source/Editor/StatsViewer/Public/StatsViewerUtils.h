// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;

namespace StatsViewerUtils
{
	/** 
	 * Helper function to get an actor from an object 
	 * @param	InObject	The UObject to try to get an associated AACtor from
	 * @returns The AActor object if any or NULL if none can be found
	 */
	STATSVIEWER_API AActor* GetActor( const TWeakObjectPtr<UObject>& InObject );

	/** 
	 * Helper function used to get an asset name from an object to display 
	 * @param	InObject	The UObject to try to get a name from
	 * @returns The name to display or an empty string if none can be found
	 */
	STATSVIEWER_API FText GetAssetName( const TWeakObjectPtr<UObject>& InObject );
}
