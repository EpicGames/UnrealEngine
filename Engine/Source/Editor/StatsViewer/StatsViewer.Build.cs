// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StatsViewer : ModuleRules
{
	public StatsViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"RHI",
				"EditorFramework",
				"UnrealEd",
				"Landscape"
			}
		);

        PrivateIncludePathModuleNames.AddRange(
			new string[] {
                "PropertyEditor",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"PropertyEditor"
			}
		);
	}
}
